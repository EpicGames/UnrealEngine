// Copyright Epic Games, Inc. All Rights Reserved.

#include <StylusInput.h>
#include <StylusInputInterface.h>
#include <StylusInputUtils.h>

#include <Algo/AnyOf.h>
#include <Algo/Find.h>

namespace UE::StylusInput
{
	namespace Private
	{
		class FStylusInputImpl
		{
		public:
			IStylusInputInstance* CreateInstance(SWindow& Window, const FName* InterfaceName, const bool bRequestedInterfaceOnly) const
			{
				// Helper for trying to create an instance for a given interface.
				auto CreateInstanceForInterface = [&Window](IStylusInputInterface& Interface) -> IStylusInputInstance*
				{
					if (IStylusInputInstance* Instance = Interface.CreateInstance(Window))
					{
						if (Instance->WasInitializedSuccessfully())
						{
							return Instance;
						}

						UE_LOGF(LogStylusInput, Error, "Stylus input instance for interface '%ls' was not initialized successfully.",
						       *Interface.GetName().ToString());

						Interface.ReleaseInstance(Instance);
					}
					else
					{
						UE_LOGF(LogStylusInput, Error, "Failed to create stylus input instance for interface '%ls'.",
							   *Interface.GetName().ToString());
					}

					return nullptr;
				};

				// Try to create an instance for the named interface.
				if (InterfaceName)
				{
					IStylusInputInterface *const * RequestedInterface = Algo::FindByPredicate(
						Interfaces, [InterfaceName](const IStylusInputInterface* Interface) { return Interface->GetName() == *InterfaceName; });

					if (RequestedInterface)
					{
						if (IStylusInputInstance* Instance = CreateInstanceForInterface(**RequestedInterface))
						{
							return Instance;
						}
					}
					else
					{
						UE_LOGF(LogStylusInput, Warning, "Requested stylus input interface '%ls' is not available.", *InterfaceName->ToString());
					}
				}

				// Otherwise, try to create an instance for any available interface, unless only the requested interface is valid.
				if (!InterfaceName || !bRequestedInterfaceOnly)
				{
					for (IStylusInputInterface* Interface : Interfaces)
					{
						if (Interface)
						{
							if (IStylusInputInstance* Instance = CreateInstanceForInterface(*Interface))
							{
								return Instance;
							}
						}
					}
				}

				UE_LOGF(LogStylusInput, Warning, "No valid stylus input interface and/or instance available.");
				return nullptr;
			}

			bool ReleaseInstance(IStylusInputInstance* Instance) const
			{
				const FName InterfaceName = Instance->GetInterfaceName();

				for (IStylusInputInterface* Interface : Interfaces)
				{
					if (Interface->GetName() == InterfaceName)
					{
						return Interface->ReleaseInstance(Instance);
					}
				}

				return false;
			}

			TArray<FName> GetAvailableInterfaces() const
			{
				TArray<FName> APINames;
				APINames.Reserve(Interfaces.Num());

				for (const IStylusInputInterface* Interface : Interfaces)
				{
					APINames.Add(Interface->GetName());
				}

				return APINames;
			}

			bool RegisterInterface(IStylusInputInterface* Interface)
			{
				if (!Interface)
				{
					UE_LOGF(Private::LogStylusInput, Warning, "Nullptr passed into StylusInput::RegisterInterface().");
					return false;
				}

				const FName InterfaceName = Interface->GetName();

				if (Algo::AnyOf(Interfaces, [&InterfaceName](const IStylusInputInterface* I) { return I->GetName() == InterfaceName; }))
				{
					UE_LOGF(Private::LogStylusInput, Error, "Interface with name '%ls' has already been registered via StylusInput::RegisterInterface().",
						   *InterfaceName.ToString())
					return false;
				}

				// HACK for temporarily making sure RealTimeStylus is the default under Windows in absence of properly handling this with editor settings and everything.
				if (InterfaceName == "RealTimeStylus")
				{
					Interfaces.EmplaceAt(0, Interface);
					return true;
				}

				Interfaces.Add(Interface);

				return true;
			}

			bool UnregisterInterface(IStylusInputInterface* Interface)
			{
				if (!Interface)
				{
					UE_LOGF(Private::LogStylusInput, Warning, "Nullptr passed into StylusInput::UnregisterInterface().");
					return false;
				}

				const FName InterfaceName = Interface->GetName();

				IStylusInputInterface** RegisteredInterface = Algo::FindByPredicate(
					Interfaces, [&InterfaceName](const IStylusInputInterface* I) { return I->GetName() == InterfaceName; });

				if (!RegisteredInterface)
				{
					UE_LOGF(Private::LogStylusInput, Error, "StylusInput::UnregisterInterface: Interface with name '%ls' has not been previously registered.",
						   *InterfaceName.ToString())
					return false;
				}

				Interfaces.RemoveSingle(*RegisteredInterface);

				return true;
			}

		private:
			TArray<IStylusInputInterface*> Interfaces;
		};
	}

	Private::FStylusInputImpl& GetImplSingleton()
	{
		static Private::FStylusInputImpl ImplSingleton;
		return ImplSingleton;
	}

	IStylusInputInstance* CreateInstance(SWindow& Window)
	{
		return GetImplSingleton().CreateInstance(Window, nullptr, false);
	}

	IStylusInputInstance* CreateInstance(SWindow& Window, const FName Interface, const bool bRequestedInterfaceOnly)
	{
		return GetImplSingleton().CreateInstance(Window, &Interface, bRequestedInterfaceOnly);
	}

	bool ReleaseInstance(IStylusInputInstance* Instance)
	{
		if (!Instance)
		{
			UE_LOGF(Private::LogStylusInput, Warning, "Nullptr passed into StylusInput::ReleaseInstance().");
			return false;
		}

		return GetImplSingleton().ReleaseInstance(Instance);
	}

	TArray<FName> GetAvailableInterfaces()
	{
		return GetImplSingleton().GetAvailableInterfaces();
	}

	bool RegisterInterface(IStylusInputInterface* Interface)
	{
		return GetImplSingleton().RegisterInterface(Interface);
	}

	bool UnregisterInterface(IStylusInputInterface* Interface)
	{
		return GetImplSingleton().UnregisterInterface(Interface);
	}
}
