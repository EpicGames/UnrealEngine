// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Templates/SharedPointer.h>

class SWindow;

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
	class IStylusInputInstance;
	class IStylusInputStylusInfo;
	class IStylusInputTabletContext;

	struct FStylusInputPacket;

	/**
	 * Creates a stylus input instance for a given window.
	 * Any call to this function that returns a valid instance pointer should have a corresponding call to @see #ReleaseInstance.
	 *
	 * @param Window The window the stylus input is mapped to.
	 * @returns The stylus input instance for the given window or a nullptr if no valid interface is available or the instance could not be initialized.
	 */
	STYLUSINPUT_API IStylusInputInstance* CreateInstance(SWindow& Window);

	/**
	 * Creates a stylus input instance for a given window and a requested interface.
	 * Any call to this function that returns a valid instance pointer should have a corresponding call to @see #ReleaseInstance.
	 *
	 * @param Window The window the stylus input is mapped to.
	 * @param Interface Request a specific interface to be used for handling stylus input.
	 * @param bRequestedInterfaceOnly Only use the requested stylus input interface, and return nullptr if this interface is not available.
	 * @returns The stylus input instance for the given window or a nullptr if no valid interface is available or the instance could not be initialized.
	 */
	STYLUSINPUT_API IStylusInputInstance* CreateInstance(SWindow& Window, FName Interface, bool bRequestedInterfaceOnly);

	/**
	 * Releases any resources held for the given stylus input instance.
	 * This function should be called for any corresponding successful call to @see #CreateInstance, e.g. when stylus input is no longer needed or the window
	 * related to the instance is destroyed.
	 *
	 * @param Instance The stylus input instance to be released.
	 * @returns True if the given instance is not a nullptr and was successfully released. 
	 */
	STYLUSINPUT_API bool ReleaseInstance(IStylusInputInstance* Instance);

	/**
	 * Defines the type of thread on which an event handler will be called by the stylus input instance.
	 */
	enum class EEventHandlerThread : int8
	{
		OnGameThread,	// The event handler will be called on the game thread.
		Asynchronous	// The event handler will be called on an unspecific thread that is not the game thread.
	};

	/**
	 * Interface for a stylus input instance.
	 */
	class IStylusInputInstance
	{
	public:
		virtual ~IStylusInputInstance() = default;

		/**
		 * Adds an event handler that gets called for any processed stylus input event.
		 * Multiple event handlers can get added for a single stylus input instance. Please note that long-running calls will block any other event handlers,
		 * the processing of additional events, and potentially the game thread itself. Thus, make sure to have an event handler call return as soon as possible.
		 *
		 * @param EventHandler The event handler that gets called for each processed stylus input event.
		 * @param Thread The type of thread the event handler calls are performed on.
		 * @returns True if the event handler was added successfully.
		 */
		virtual bool AddEventHandler(IStylusInputEventHandler* EventHandler, EEventHandlerThread Thread) = 0;

		/**
		 * Removes an event handler from this stylus input instances, and frees up all associated resources.
		 * The given event handler should have been added previously via a call to @see AddEventHandler.
		 *
		 * @param EventHandler The event handler to be released.
		 * @returns True if the event handler was removed successfully.
		 */
		virtual bool RemoveEventHandler(IStylusInputEventHandler* EventHandler) = 0;

		/**
		 * Provides the tablet context information for a given tablet context ID.
		 * This function should only be called with tablet context IDs recently provided by the stylus input instance,
		 * e.g. within a @see FStylusPacket as part of an event handler callback.
		 *
		 * Calls to this function are thread-safe, i.e. can be made from an event handler callback running on any thread.
		 *
		 * @param TabletContextID The unique identifier of the tablet context as provided by this stylus input instance via an event handler callback. 
		 * @returns Pointer to tablet context information or nullptr if the tablet context ID was invalid or an error occured. 
		 */
		virtual const TSharedPtr<IStylusInputTabletContext> GetTabletContext(uint32 TabletContextID) = 0;

		/**
		 * Provides the stylus a.k.a. cursor or pen information for a given stylus ID.
		 * This function should only be called with stylus IDs recently provided by the stylus input instance,
		 * e.g. within a @see FStylusPacket as part of an event handler callback.
		 *
		 * Calls to this function are thread-safe, i.e. can be made from an event handler callback running on any thread.
		 *
		 * @param StylusID The unique identifier of the stylus/cursor/pen as provided by this stylus input instance via an event handler callback. 
		 * @returns Pointer to stylus information or nullptr if the stylus ID was invalid or an error occured. 
		 */
		virtual const TSharedPtr<IStylusInputStylusInfo> GetStylusInfo(uint32 StylusID) = 0;

		/**
		 * Provides the approximate number of stylus input packets processed per second for the given thread type for diagnostic purposes.
		 * The result will only be meaningful if at least one event handler was added for the given thread type.
		 *
		 * @param Thread Selects the set of event handlers, i.e. number of packets processed on the game thread versus processed asynchronously.
		 * @returns Approximate number of stylus input packets processed per second, or -1 if there is no valid data available.
		 */
		virtual float GetPacketsPerSecond(EEventHandlerThread Thread) const { return -1.0f; }

		/**
		 * Provides the name of the interface used for this instance. 
		 * 
		 * @returns Name of the interface.
		 */
		virtual FName GetInterfaceName() = 0;

		/**
		 * Provides the name of this instance for diagnostic purposes. 
		 * 
		 * @returns Name of this instance.
		 */
		virtual FText GetName() = 0;

		/**
		 * 
		 * @returns true if this instance was initialized successfully and able to provide stylus input events.
		 */
		virtual bool WasInitializedSuccessfully() const = 0;
	};

	/**
	 * Interface for a stylus input event handler.
	 */
	class IStylusInputEventHandler
	{
	public:
		virtual ~IStylusInputEventHandler() = default;

		/**
		 * Returns the name of the event handler. This is used internally for diagnostic purposes and reporting warnings and errors.
		 *
		 * @returns Name of the event handler.
		 */
		virtual FString GetName() = 0;

		/**
		 * Callback for each packet processed by the stylus input instance.
		 * This function is called on the game thread or asynchronously to the game thread depending on how the event handler was added to the stylus input
		 * instance. The implementation of this function is responsible for any thread synchronization that might be necessary. 
		 *
		 * @param Packet The packet being processed.
		 * @param Instance The stylus input instance that sent the packet.
		 */
		virtual void OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance* Instance) = 0;

		/**
		 * Callback for each debug event sent by the stylus input instance for diagnostic purposes.
		 * This function is called on the game thread or asynchronously to the game thread depending on how the event handler was added to the stylus input
		 * instance. The implementation of this function is responsible for any thread synchronization that might be necessary.
		 * The base class implementation of this function does nothing.
		 *
		 * @param Message The debug message sent by the stylus input instance.
		 * @param Instance The stylus input instance that sent the debug message.
		 */
		virtual void OnDebugEvent(const FString& Message, IStylusInputInstance* Instance) {}
	};
}
