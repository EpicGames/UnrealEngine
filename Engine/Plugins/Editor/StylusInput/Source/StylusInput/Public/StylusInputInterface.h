// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <UObject/NameTypes.h>

class SWindow;

namespace UE::StylusInput
{
	class IStylusInputInstance;
	class IStylusInputInterface;

	/**
	 * Provides the names of all currently available stylus input interfaces. 
	 *
	 * @return Names of available interfaces. 
	 */
	STYLUSINPUT_API TArray<FName> GetAvailableInterfaces();

	/**
	 * Adds an interface to the StylusInput module.
	 * 
	 * Calls to this function are only allowed after the "Default" loading phase. This can be ensured by loading modules providing stylus input interfaces in
	 * the "PostDefault" loading phase.
	 *
	 * @param Interface The interface to be registered.
	 */
	STYLUSINPUT_API bool RegisterInterface(IStylusInputInterface* Interface);

	/**
	 * Removes an interface from the StylusInput module.
	 * 
	 * The StylusInput module does not deallocate the provided interface.
	 * 
	 * @param Interface The interface to be unregistered.
	 */
	STYLUSINPUT_API bool UnregisterInterface(IStylusInputInterface* Interface);

	/**
	 * Interface for platform APIs that provide stylus input data.
	 */
	class IStylusInputInterface
	{
	public:
		virtual ~IStylusInputInterface() = default;

		/**
		 * Provides the name of the interface; this needs to be unique among all interfaces. 
		 * 
		 * @returns Name of the interface.
		 */
		virtual FName GetName() const = 0;

		/**
		 * Implementation of /see StylusInput::CreateInstance for this interface. 
		 */
		virtual IStylusInputInstance* CreateInstance(SWindow& Window) = 0;

		/**
		 * Implementation of /see StylusInput::ReleaseInstance for this interface. 
		 */
		virtual bool ReleaseInstance(IStylusInputInstance* Instance) = 0;
	};
}
