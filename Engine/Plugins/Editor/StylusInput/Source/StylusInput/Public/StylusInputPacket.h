// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include <HAL/Platform.h>

namespace UE::StylusInput
{
	/**
	 * Flags for the status of the stylus pen.
	 */
	enum class EPenStatus : uint8
	{
		None				= 0b00000000,	///< There is no pen flag.
		CursorIsTouching	= 0b00000001,	///< The pen cursor is touching the drawing surface.
		CursorIsInverted	= 0b00000010,	///< The pen cursor is inverted, e.g. the eraser end of the pen is pointing toward the drawing surface.
		NotUsed				= 0b00000100,	///< Not used.
		BarrelButtonPressed = 0b00001000	///< The barrel button is pressed.
	};

	/**
	 * Describes the interaction between pen and drawing surface (a.k.a. digitizer) for which the packet was generated.
	 */
	enum class EPacketType : uint8
	{
		Invalid,		///< The packet is not valid.
		OnDigitizer,	///< The packet was generated while the pen's cursor is touching the drawing surface.
		AboveDigitizer, ///< The packet was generated while the pen's cursor is hovering in proximity above the drawing surface.
		StylusDown,		///< The packet was generated when the pen's cursor started touching the drawing surface.
		StylusUp		///< The packet was generated when the pen's cursor stopped touching the drawing surface.
	};

	/**
	 * Data for an interaction between the pen and the drawing surface.
	 *
	 * Note that not all devices support all properties. The set of valid properties can be queried via @see IStylusInputTabletContext#GetSupportedProperties,
	 * and a description of all properties is available in @see ETabletSupportedProperties.
	 */
	struct FStylusInputPacket
	{
		// Metadata
		uint32 TabletContextID = 0;					///< Unique identifier for the tablet context that created the packet.
		uint32 CursorID = 0;						///< Unique identifier for the stylus/cursor/pen that was used to create the packet.
		EPacketType Type = EPacketType::Invalid;	///< The type of interaction between pen and drawing surface that created the packet.
		EPenStatus PenStatus = EPenStatus::None;	///< The status of the stylus/cursor/pen when the packet was created.

		// Properties
		float X = 0.0f;
		float Y = 0.0f;
		float Z = 0.0f;
		uint32 TimerTick = 0;
		uint32 SerialNumber = 0;
		float NormalPressure = 0.0f;
		float TangentPressure = 0.0f;
		float ButtonPressure = 0.0f;
		float XTiltOrientation = 0.0f;
		float YTiltOrientation = 0.0f;
		float AzimuthOrientation = 0.0f;
		float AltitudeOrientation = 0.0f;
		float TwistOrientation = 0.0f;
		float PitchRotation = 0.0f;
		float RollRotation = 0.0f;
		float YawRotation = 0.0f;
		float Width = 0.0f;
		float Height = 0.0f;
		float FingerContactConfidence = 0.0f;
		int32 DeviceContactID = 0;
	};

	/* Bitwise operators for EPenStatus to be used as flags without additional casts. */

	inline EPenStatus operator|(EPenStatus A, EPenStatus B)
	{
		return static_cast<EPenStatus>(static_cast<std::underlying_type_t<EPenStatus>>(A) | static_cast<std::underlying_type_t<EPenStatus>>(B));
	}

	inline EPenStatus operator&(EPenStatus A, EPenStatus B)
	{
		return static_cast<EPenStatus>(static_cast<std::underlying_type_t<EPenStatus>>(A) & static_cast<std::underlying_type_t<EPenStatus>>(B));
	}

	inline EPenStatus operator^(EPenStatus A, EPenStatus B)
	{
		return static_cast<EPenStatus>(static_cast<std::underlying_type_t<EPenStatus>>(A) ^ static_cast<std::underlying_type_t<EPenStatus>>(B));
	}

	inline EPenStatus operator~(EPenStatus A)
	{
		return static_cast<EPenStatus>(~static_cast<std::underlying_type_t<EPenStatus>>(A));
	}
}
