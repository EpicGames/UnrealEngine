// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include <Containers/UnrealString.h>

namespace UE::StylusInput
{
	/**
	 * Flags describing tablet hardware capabilities.
	 */
	enum class ETabletHardwareCapabilities : uint8
	{
		None					= 0,
		Integrated				= 1 << 0, ///< The digitizer is integrated with the display.
		CursorMustTouch			= 1 << 1, ///< The cursor must be in physical contact with the device to report position.
		HardProximity			= 1 << 2, ///< The device can generate in-air packets when the cursor is in the physical detection range (proximity) of the device.
		CursorsHavePhysicalIds	= 1 << 3  ///< The device can uniquely identify the active cursor.
	};

	/**
	 * Flags denoting the properties supported by a tablet.
	 */
	enum class ETabletSupportedProperties : uint32
	{
		None				    = 0,
		X					    = 1 <<  0, ///< The x-coordinate in the coordinate space of the window that was used to create the stylus input instance. Each packet contains this property by default.
		Y					    = 1 <<  1, ///< The y-coordinate in the coordinate space of the window that was used to create the stylus input instance. Each packet contains this property by default.
		Z					    = 1 <<  2, ///< The z-coordinate or distance of the pen tip from the tablet surface. The TabletPropertyMetricUnit enumeration  type determines the unit of measurement for this property.
		PacketStatus		    = 1 <<  3, ///< Contains one or more of the following flag values: The cursor is touching the drawing surface (Value = 1). The cursor is inverted. For example, the eraser end of the pen is pointing toward the surface (Value = 2). Not used (Value = 4). The barrel button is pressed (Value = 8).
		TimerTick			    = 1 <<  4, ///< The time the packet was generated in milliseconds since system start.
		SerialNumber		    = 1 <<  5, ///< The packet property for identifying the packet. This is the same value you use to retrieve the packet from the packet queue.
		NormalPressure		    = 1 <<  6, ///< The pressure of the pen tip perpendicular to the tablet surface. The greater the pressure on the pen tip, the  more ink that is drawn.
		TangentPressure		    = 1 <<  7, ///< The pressure of the pen tip along the plane of the tablet surface.
		ButtonPressure		    = 1 <<  8, ///< The pressure on a pressure sensitive button.
		XTiltOrientation	    = 1 <<  9, ///< The angle between the y,z-plane and the pen and y-axis plane. Applies to a pen cursor. The value is 0 when the pen is perpendicular to the drawing surface and is positive when the pen is to the right of perpendicular.
		YTiltOrientation	    = 1 << 10, ///< The angle between the x,z-plane and the pen and x-axis plane. Applies to a pen cursor. The value is 0 when the pen is perpendicular to the drawing surface and is positive when the pen is upward or away from the user.
		AzimuthOrientation	    = 1 << 11, ///< The clockwise rotation of the cursor about the z-axis through a full circular range.
		AltitudeOrientation     = 1 << 12, ///< The angle between the axis of the pen and the surface of the tablet. The value is 0 when the pen is parallel to the surface and 90 when the pen is perpendicular to the surface. The values are negative when the pen is inverted.
		TwistOrientation	    = 1 << 13, ///< The clockwise rotation of the cursor about its own axis.
		PitchRotation		    = 1 << 14, ///< The packet property that indicates whether the tip is above or below a horizontal line that is perpendicular to the writing surface. Note: This requires a 3D digitizer. The value is positive if the tip is above the line and negative if it is below the line. For example, if you hold the pen in front of you and write on an imaginary wall, the pitch is positive if the tip is above a line extending from you to the wall.
		RollRotation		    = 1 << 15, ///< The clockwise rotation of the pen around its own axis. Note: This requires a 3D digitizer.
		YawRotation			    = 1 << 16, ///< The angle of the pen to the left or right around the center of its horizontal axis when the pen is horizontal. Note: This requires a 3D digitizer. If you hold the pen in front of you and write on an imaginary wall, zero yaw indicates that the pen is perpendicular to the wall. The value is negative if the tip is to the left of perpendicular and positive if the tip is to the right of perpendicular.
		Width				    = 1 << 17, ///< The width of the contact area on a touch digitizer.
		Height					= 1 << 18, ///< The height of the contact area on a touch digitizer.
		FingerContactConfidence = 1 << 19, ///< The level of confidence that there was finger contact on a touch digitizer.
		DeviceContactID			= 1 << 20, ///< The device contact identifier for a packet, e.g. to identify individual fingers.
	};

	/**
	 * Identifies and describes the capabilities of a tablet drawing surface.
	 */
	class IStylusInputTabletContext
	{
	public:
		virtual ~IStylusInputTabletContext() = default;

		/**
		 * Returns the unique identifier for the tablet context.
		 * The returned ID is only unique and identical for a given stylus input instance, i.e. the same tablet device might have different IDs for different
		 * stylus input instances, and different tablet devices might have the same ID in two separate stylus input instances.
		 *
		 * @returns The unique identifier for the tablet context for the respective stylus input instance.
		 */
		virtual uint32 GetID() const = 0;

		/**
		 * Returns the name of the tablet context.
		 * This usually allows to identify the tablet device based on the hardware product name and model.
		 *
		 * @returns The name of the tablet context.
		 */
		virtual FString GetName() const = 0;

		/**
		 * Returns the dimensions of the digitizer surface in device coordinates.
		 * This mainly gives an indication of the spatial resolution of the device, and it is not related to the coordinate space of the window that was used to
		 * create the stylus input instance.
		 *
		 * @returns The dimensions of the digitizer surface.
		 */
		virtual FIntRect GetInputRectangle() const = 0;

		/**
		 * Returns the hardware capabilities of the device associated with the tablet context.
		 *
		 * @return The hardware capability flags.
		 */
		virtual ETabletHardwareCapabilities GetHardwareCapabilities() const = 0;

		/**
		 * Returns the supported properties for a tablet context.
		 * Only explicitly supported properties for a tablet context provide valid values in the packet data with the respective tablet context ID.
		 *
		 * @returns The supported properties flags.
		 */
		virtual ETabletSupportedProperties GetSupportedProperties() const = 0;
	};

	/**
	 * Description of a button on a stylus pen.
	 */
	class IStylusInputStylusButton
	{
	public:
		virtual ~IStylusInputStylusButton() = default;

		/**
		 * @returns The GUID string for the button. 
		 */
		virtual FString GetID() const = 0;

		/**
		 * @returns  The name of the button.
		 */
		virtual FString GetName() const = 0;
	};

	/**
	 * Description of a stylus pen.
	 */
	class IStylusInputStylusInfo
	{
	public:
		virtual ~IStylusInputStylusInfo() = default;

		/**
		 * Returns the identifier for the type of stylus being used.
		 * Note that different hardware pens can have the same ID within a tablet context.
		 *
		 * @returns Identifier of the stylus pen.
		 */
		virtual uint32 GetID() const = 0;

		/**
		 * @returns The name of the stylus pen.
		 */
		virtual FString GetName() const = 0;

		/**
		 * Returns the number of buttons on the stylus.
		 * Note that some hardware buttons might not show up in the stylus info provided by the tablet context.
		 *
		 * @returns The number of buttons on the stylus.
		 */
		virtual uint32 GetNumButtons() const = 0;

		/**
		 * Returns a description of the button for a given index.
		 *
		 * @param Index A number in [0, @see #GetNumButtons).
		 * @returns Pointer to a stylus button.
		 */
		virtual const IStylusInputStylusButton* GetButton(int32 Index) const = 0;
	};

	/* Bitwise operators for enums to be used as flags without additional casts. */

	inline ETabletHardwareCapabilities operator|(ETabletHardwareCapabilities A, ETabletHardwareCapabilities B)
	{
		return static_cast<ETabletHardwareCapabilities>(static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(A) |
			static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(B));
	}

	inline ETabletHardwareCapabilities operator&(ETabletHardwareCapabilities A, ETabletHardwareCapabilities B)
	{
		return static_cast<ETabletHardwareCapabilities>(static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(A) &
			static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(B));
	}

	inline ETabletHardwareCapabilities operator^(ETabletHardwareCapabilities A, ETabletHardwareCapabilities B)
	{
		return static_cast<ETabletHardwareCapabilities>(static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(A) ^
			static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(B));
	}

	inline ETabletHardwareCapabilities operator~(ETabletHardwareCapabilities A)
	{
		return static_cast<ETabletHardwareCapabilities>(~static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(A));
	}

	inline ETabletSupportedProperties operator|(ETabletSupportedProperties A, ETabletSupportedProperties B)
	{
		return static_cast<ETabletSupportedProperties>(static_cast<std::underlying_type_t<ETabletSupportedProperties>>(A) |
			static_cast<std::underlying_type_t<ETabletSupportedProperties>>(B));
	}

	inline ETabletSupportedProperties operator&(ETabletSupportedProperties A, ETabletSupportedProperties B)
	{
		return static_cast<ETabletSupportedProperties>(static_cast<std::underlying_type_t<ETabletSupportedProperties>>(A) &
			static_cast<std::underlying_type_t<ETabletSupportedProperties>>(B));
	}

	inline ETabletSupportedProperties operator^(ETabletSupportedProperties A, ETabletSupportedProperties B)
	{
		return static_cast<ETabletSupportedProperties>(static_cast<std::underlying_type_t<ETabletSupportedProperties>>(A) ^
			static_cast<std::underlying_type_t<ETabletSupportedProperties>>(B));
	}

	inline ETabletSupportedProperties operator~(ETabletSupportedProperties A)
	{
		return static_cast<ETabletSupportedProperties>(~static_cast<std::underlying_type_t<ETabletSupportedProperties>>(A));
	}
}
