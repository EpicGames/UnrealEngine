// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Logging/LogMacros.h>
#include <Misc/ScopeRWLock.h>
#include <Templates/SharedPointer.h>

namespace UE::StylusInput::Private
{
	STYLUSINPUT_API DECLARE_LOG_CATEGORY_EXTERN(LogStylusInput, Log, All)

	inline void LogError(const FString& Preamble, const FString& Message)
	{
		UE_LOGF(LogStylusInput, Error, "%ls: %ls", *Preamble, *Message);
	}

	inline void LogWarning(const FString& Preamble, const FString& Message)
	{
		UE_LOGF(LogStylusInput, Warning,  "%ls: %ls", *Preamble, *Message);
	}

	inline void Log(const FString& Preamble, const FString& Message)
	{
		UE_LOGF(LogStylusInput, Log,  "%ls: %ls", *Preamble, *Message);
	}

	inline void LogVerbose(const FString& Preamble, const FString& Message)
	{
		UE_LOGF(LogStylusInput, Verbose,  "%ls: %ls", *Preamble, *Message);
	}

	template <typename DataType, bool ThreadSafe>
	class TSharedRefDataContainer
	{
	public:
		TSharedRef<DataType> Add(uint32 ID)
		{
			WriteScopeLockType WriteLock(RWLock);
			return Data.Emplace_GetRef(MakeShared<DataType>(ID));
		}

		void AddOrReplace(TSharedPtr<DataType>&& Item)
		{
			if (Item.IsValid())
			{
				TSharedRef<DataType>* ExistingItem = Data.FindByPredicate([ID = Item->ID](const TSharedRef<DataType>& Context)
				{
					return Context->ID == ID;
				});

				WriteScopeLockType WriteLock(RWLock);
				if (ExistingItem != nullptr)
				{
					*ExistingItem = MoveTemp(Item).ToSharedRef();
				}
				else
				{
					Data.Emplace(MoveTemp(Item).ToSharedRef());
				}
			}
		}

		bool Contains(uint32 ID) const
		{
			ReadScopeLockType ReadLock(RWLock);
			const TSharedRef<DataType>* Item = Data.FindByPredicate([ID](const TSharedRef<DataType>& Context)
			{
				return Context->ID == ID;
			});
			return Item != nullptr;
		}

		TSharedPtr<DataType> Get(uint32 ID) const
		{
			ReadScopeLockType ReadLock(RWLock);
			const TSharedRef<DataType>* Item = Data.FindByPredicate([ID](const TSharedRef<DataType>& Context)
			{
				return Context->ID == ID;
			});
			return Item ? Item->ToSharedPtr() : TSharedPtr<DataType>{};
		}

		bool Remove(uint32 ID)
		{
			int32 Index;
			{
				ReadScopeLockType ReadLock(RWLock);
				Index = Data.IndexOfByPredicate([ID](const TSharedRef<DataType>& Tc) { return Tc->ID == ID; });
				if (Index == INDEX_NONE)
				{
					return false;
				}
			}
			WriteScopeLockType WriteLock(RWLock);
			Data.RemoveAt(Index, EAllowShrinking::No);
			return true;
		}

		void Clear()
		{
			WriteScopeLockType WriteLock(RWLock);
			Data.Reset();
		}

		void Update(const TSharedRefDataContainer<DataType, false>& InData)
		{
			WriteScopeLockType WriteLock(RWLock);
			Data.Reset();
			for (uint32 Index = 0, Num = InData.Num(); Index < Num; ++Index)
			{
				Data.Emplace(InData[Index]);
			}
		}

		uint32 Num() const { return Data.Num(); }

		const TSharedRef<DataType>& operator[](const uint32 Index) const { return Data[Index]; }

	private:

		struct FNoLock {};
		struct FScopeNoLock { explicit FScopeNoLock(FNoLock&) {} };

		using LockType = std::conditional_t<ThreadSafe, FRWLock, FNoLock>;
		using ReadScopeLockType = std::conditional_t<ThreadSafe, FReadScopeLock, FScopeNoLock>;
		using WriteScopeLockType = std::conditional_t<ThreadSafe, FWriteScopeLock, FScopeNoLock>;

		UE_NO_UNIQUE_ADDRESS mutable LockType RWLock;
		TArray<TSharedRef<DataType>> Data;
	};
}
