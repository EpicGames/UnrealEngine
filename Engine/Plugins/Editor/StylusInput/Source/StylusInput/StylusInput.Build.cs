// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StylusInput : ModuleRules
	{
		public StylusInput(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				[
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"Slate",
					"UnrealEd",
				]
			);

			PrivateDependencyModuleNames.AddRange(
				[
					"SlateCore",
				]
			);
		}
	}
}
