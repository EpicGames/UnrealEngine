// Copyright Epic Games, Inc. All Rights Reserved.

#include "StylusInputDebugPaintWidget.h"

#include <StylusInputPacket.h>
#include <Brushes/SlateColorBrush.h>
#include <Framework/Application/SlateApplication.h>
#include <Styling/AppStyle.h>

namespace UE::StylusInput::DebugWidget
{
	int32 DrawBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId)
	{
		const FLinearColor BackgroundColor(FVector3f(0.05f));
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")),
		                           ESlateDrawEffect::None, BackgroundColor);
		return ++LayerId;
	}

	void DrawCircle(FVector2f Position, const float Size, const FLinearColor& Color, const FGeometry& AllottedGeometry,
	                FSlateWindowElementList& OutDrawElements, int32 LayerId)
	{
		const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry(Slate::FDeprecateVector2DParameter(Size, Size), FSlateLayoutTransform(Position));

		FSlateColorBrush ColorBrush = FLinearColor::White;
		ColorBrush.DrawAs = ESlateBrushDrawType::Type::RoundedBox;

		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, PaintGeometry, &ColorBrush, ESlateDrawEffect::None, Color);
	}

	void SStylusInputDebugPaintWidget::Construct(const FArguments& Args)
	{
		PaintPackets.Reserve(1024);
	}

	void SStylusInputDebugPaintWidget::Add(const FStylusInputPacket& Packet)
	{
		PaintPackets.Emplace(FPaintPacket{
			FVector2f{Packet.X, Packet.Y}, Packet.NormalPressure, Packet.PenStatus, FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64())
		});
	}

	int32 SStylusInputDebugPaintWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	                                            FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle,
	                                            bool bParentEnabled) const
	{
		LayerId = DrawBackground(AllottedGeometry, OutDrawElements, LayerId);
		LayerId = DrawPaintPackets(AllottedGeometry, OutDrawElements, LayerId);

		return LayerId;
	}

	TOptional<EMouseCursor::Type> SStylusInputDebugPaintWidget::GetCursor() const
	{
		return EMouseCursor::Crosshairs;
	}

	int32 SStylusInputDebugPaintWidget::DrawPaintPackets(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		if (PaintPackets.IsEmpty())
		{
			return LayerId;
		}

		constexpr float MinSize = 2.0f;
		constexpr float MaxSize = 15.0f;
		constexpr double FadeOutTimeMSPenDown = 3000.0;
		constexpr double FadeOutTimeMSPenUp = 1000.0;

		constexpr FLinearColor ColorPenDown = {0.0f, 0.2f, 1.0f};
		constexpr FLinearColor ColorPenDownInverted = {1.0f, 0.1f, 0.0f};
		constexpr FLinearColor ColorPenUp = {0.5f, 0.5f, 0.5f};

		Slate::FDeprecateVector2DResult WindowPosition = FVector2f::ZeroVector;
		float DPIScaleFactor = 1.0f;

		const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ensure(Window))
		{
			WindowPosition = Window->GetPaintSpaceGeometry().GetAbsolutePosition();
			DPIScaleFactor = Window ? Window->GetDPIScaleFactor() : 1.0f;
		}

		const Slate::FDeprecateVector2DResult PaintWidgetPosition = GetPaintSpaceGeometry().GetAbsolutePosition();
		const FDeprecateSlateVector2D PositionOffset = PaintWidgetPosition - WindowPosition;

		const double CurrentTimeMS = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		auto GetOpacity = [FadeOutTimeMSPenDown, FadeOutTimeMSPenUp, CurrentTimeMS](const FPaintPacket& PaintPacket) -> float
		{
			const bool bPenIsDown = (PaintPacket.PenStatus & EPenStatus::CursorIsTouching) != EPenStatus::None;
			const double DeltaTimeMS = CurrentTimeMS - PaintPacket.TimeAddedMS;
			const double FadeOutTimeMS = bPenIsDown ? FadeOutTimeMSPenDown : FadeOutTimeMSPenUp;
			return static_cast<float>((FadeOutTimeMS - DeltaTimeMS) / FadeOutTimeMS);
		};

		OutDrawElements.PushClip(FSlateClippingZone(AllottedGeometry));
		{
			for (const FPaintPacket& PaintPacket : PaintPackets)
			{
				if (const float Opacity = GetOpacity(PaintPacket); Opacity > 0.0f)
				{
					const bool bPenIsDown = (PaintPacket.PenStatus & EPenStatus::CursorIsTouching) != EPenStatus::None;
					const bool bPenIsInverted = (PaintPacket.PenStatus & EPenStatus::CursorIsInverted) != EPenStatus::None;

					FLinearColor Color = bPenIsDown ? (bPenIsInverted ? ColorPenDownInverted : ColorPenDown) : ColorPenUp;
					Color.A = Opacity;

					const float Size = FMath::Max(MinSize, PaintPacket.NormalPressure * MaxSize);

					const FVector2f TransformedPosition = (PaintPacket.Position - PositionOffset - Size / 2.0f) / DPIScaleFactor;

					DrawCircle(TransformedPosition, Size, Color, AllottedGeometry, OutDrawElements, LayerId);
				}
			}
		}
		OutDrawElements.PopClip();

		PaintPackets.RemoveAll([&GetOpacity](const FPaintPacket& PaintPacket) { return GetOpacity(PaintPacket) <= 0.0f; });

		return ++LayerId;
	}
}
