// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputPacket.h>
#include <Widgets/SCompoundWidget.h>
#include <Containers/RingBuffer.h>

namespace UE::StylusInput
{
	struct FStylusInputPacket;
}

namespace UE::StylusInput::DebugWidget
{
	class SStylusInputDebugPaintWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SStylusInputDebugPaintWidget)
			{
			}

		SLATE_END_ARGS()

		void Construct(const FArguments& Args);

		void Add(const FStylusInputPacket& Packet);

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
							  FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

		virtual TOptional<EMouseCursor::Type> GetCursor() const;

	private:
		struct FPaintPacket
		{
			FVector2f Position = FVector2f::ZeroVector;
			float NormalPressure = 0.0f;
			EPenStatus PenStatus = EPenStatus::None;
			double TimeAddedMS = 0.0;
		};

		int32 DrawPaintPackets(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

		mutable TRingBuffer<FPaintPacket> PaintPackets;
	};
}
