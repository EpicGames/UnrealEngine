// Copyright Epic Games, Inc. All Rights Reserved.

#include "StylusInputDebugWidget.h"

#include <StylusInput.h>
#include <StylusInputInterface.h>
#include <StylusInputTabletContext.h>
#include <Framework/Application/SlateApplication.h>
#include <Framework/MultiBox/MultiBoxBuilder.h>
#include <Widgets/SBoxPanel.h>
#include <Widgets/SOverlay.h>
#include <Widgets/Input/SComboButton.h>
#include <Widgets/Layout/SSplitter.h>
#include <Widgets/Text/SMultiLineEditableText.h>
#include <Widgets/Text/STextBlock.h>

#include "StylusInputDebugPaintWidget.h"

#define LOCTEXT_NAMESPACE "StylusInputDebugWidget"

namespace UE::StylusInput::DebugWidget
{
	DECLARE_LOG_CATEGORY_EXTERN(LogStylusInputDebugWidget, Log, All)

	DEFINE_LOG_CATEGORY(LogStylusInputDebugWidget);

	inline void LogError(const FString& Message)
	{
		UE_LOGF(LogStylusInputDebugWidget, Error, "%ls", *Message);
	}

	inline void LogWarning(const FString& Message)
	{
		UE_LOGF(LogStylusInputDebugWidget, Warning, "%ls", *Message);
	}

	inline void LogVerbose(const FString& Message)
	{
		UE_LOGF(LogStylusInputDebugWidget, Verbose, "%ls", *Message);
	}

	void SStylusInputDebugWidget::Construct(const FArguments&)
	{
		Interface = []
		{
			const TArray<FName> Interfaces = GetAvailableInterfaces();
			return Interfaces.IsEmpty() ? NAME_None : Interfaces[0];
		}();

		// One-time timer to initialize stylus plugin as soon as the widget is up.
		RegisterActiveTimer(
			0.0f,
			FWidgetActiveTimerDelegate::CreateLambda([this](double, float)
			{
				AcquireStylusInput();
				RegisterEventHandler();
				return EActiveTimerReturnType::Stop;
			}));

		TSharedPtr<SVerticalBox> TopLeftOverlay;
		TSharedPtr<SVerticalBox> BottomLeftOverlay;

		ChildSlot
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(2.0f)
				+ SSplitter::Slot()
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SAssignNew(PaintWidget, SStylusInputDebugPaintWidget)
					]
					+ SOverlay::Slot()
					.Padding(2.0f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SAssignNew(TopLeftOverlay, SVerticalBox)
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNullWidget::NullWidget
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SComboButton)
									.OnGetMenuContent(this, &SStylusInputDebugWidget::GetInterfaceMenu)
									.ButtonContent()
									[
										SNew(STextBlock)
										.Text_Lambda([&Interface = Interface]
										{
											return Interface == NAME_None ? FText::FromString("<not available>") : FText::FromName(Interface);
										})
									]
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SComboButton)
									.OnGetMenuContent(this, &SStylusInputDebugWidget::GetEventHandlerThreadMenu)
									.ButtonContent()
									[
										SNew(STextBlock)
										.Text_Lambda([&EventHandlerThread = EventHandlerThread]
										{
											return FText::FromString(
												EventHandlerThread == EEventHandlerThread::Asynchronous ? "Asynchronous" : "On Game Thread");
										})
									]
								]
								+ SVerticalBox::Slot()
								.FillHeight(1.0f)
								[
									SNullWidget::NullWidget
								]
							]
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNullWidget::NullWidget
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SAssignNew(BottomLeftOverlay, SVerticalBox)
							]
						]
					]
				]
				+ SSplitter::Slot()
				.Value(0.2f)
				[
					SNew(SMultiLineEditableText)
					.IsReadOnly(true)
					.Text_Lambda([&DebugMessages = DebugMessages] { return FText::FromString(DebugMessages); })
					.VScrollBar(SNew(SScrollBar).Orientation(Orient_Vertical).AlwaysShowScrollbar(true).Thickness(10))
				]
			];

		auto AddToOverlay = [](const TSharedPtr<SVerticalBox>& Box, TFunction<FText()>&& Text)
		{
			Box->AddSlot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text_Lambda(MoveTemp(Text))
				];
		};

		auto AddToOverlayWithVisibility = [&bIsSet = LastPacketData.bIsSet, &TabletContext = LastPacketData.TabletContext](const TSharedPtr<SVerticalBox>& Box, ETabletSupportedProperties Property, TFunction<FText()>&& Text)
		{
			Box->AddSlot()
				.AutoHeight()
				[
					SNew(STextBlock)
						.Visibility_Lambda([&bIsSet, &TabletContext, Property]
							{
								return bIsSet && TabletContext && (TabletContext->GetSupportedProperties() & Property)
								       != ETabletSupportedProperties::None
									       ? EVisibility::Visible
									       : EVisibility::Collapsed;
							})
						.Text_Lambda(MoveTemp(Text))
				];
		};

		AddToOverlay(TopLeftOverlay,
					 [&StylusInput = StylusInput]
					 {
						 return StylusInput
							        ? FText::Format(LOCTEXT("Instance", "Instance: {0}"), StylusInput->GetName())
							        : FText();
					 });

		AddToOverlay(TopLeftOverlay,
					 [&TabletContext = LastPacketData.TabletContext]
					 {
						 return TabletContext
							        ? FText::Format(LOCTEXT("TabletID", "Tablet ID: {0}"), TabletContext->GetID())
							        : FText();
					 });

		AddToOverlay(TopLeftOverlay,
					 [&TabletContext = LastPacketData.TabletContext]
					 {
						 return TabletContext
									? FText::Format(
										LOCTEXT("TabletName", "Tablet Name: {0}"),
										FText::FromString([&TabletContext]
										{
											const FString Name = TabletContext->GetName();
											return !Name.IsEmpty() ? Name : "<empty>";
										}()))
									: FText();
					 });

		AddToOverlay(TopLeftOverlay,
		             [&TabletContext = LastPacketData.TabletContext]
		             {
			             if (TabletContext)
			             {
				             const FIntRect InputRectangle = TabletContext->GetInputRectangle();
				             return FText::Format(
					             LOCTEXT("TabletInputRectangle", "Tablet Input Rectangle: ({0}, {1}) x ({2}, {3})"),
					             InputRectangle.Min.X, InputRectangle.Min.Y, InputRectangle.Max.X, InputRectangle.Max.Y);
			             }
			             return FText();
		             });

		AddToOverlay(TopLeftOverlay,
		             [&TabletContext = LastPacketData.TabletContext]
		             {
			             if (TabletContext)
			             {
				             const ETabletHardwareCapabilities Capabilities = TabletContext->GetHardwareCapabilities();

				             FString CapabilitiesStr;
				             if (Capabilities == ETabletHardwareCapabilities::None)
				             {
					             CapabilitiesStr = "None";
				             }
				             else
				             {
					             if ((Capabilities & ETabletHardwareCapabilities::Integrated) != ETabletHardwareCapabilities::None)
					             {
						             CapabilitiesStr += "Integrated";
					             }
					             if ((Capabilities & ETabletHardwareCapabilities::CursorMustTouch) != ETabletHardwareCapabilities::None)
					             {
						             CapabilitiesStr += CapabilitiesStr.IsEmpty() ? "CursorMustTouch" : ", CursorMustTouch";
					             }
					             if ((Capabilities & ETabletHardwareCapabilities::HardProximity) != ETabletHardwareCapabilities::None)
					             {
						             CapabilitiesStr += CapabilitiesStr.IsEmpty() ? "HardProximity" : ", HardProximity";
					             }
					             if ((Capabilities & ETabletHardwareCapabilities::CursorsHavePhysicalIds) != ETabletHardwareCapabilities::None)
					             {
						             CapabilitiesStr += CapabilitiesStr.IsEmpty() ? "CursorsHavePhysicalIds" : ", CursorsHavePhysicalIds";
					             }
				             }

				             if (CapabilitiesStr.IsEmpty())
				             {
					             CapabilitiesStr = FString::Format(TEXT("Unknown ({0})"), {
						                                               static_cast<std::underlying_type_t<ETabletHardwareCapabilities>>(
							                                               Capabilities)
					                                               });
				             }

				             return FText::Format(
					             LOCTEXT("TabletHardwareCapabilities", "Tablet Hardware Capabilities: {0}"), FText::FromString(CapabilitiesStr));
			             }
			             return FText();
		             });

		AddToOverlay(TopLeftOverlay,
		             [&StylusInfo = LastPacketData.StylusInfo]
		             {
			             return StylusInfo
				                    ? FText::Format(LOCTEXT("StylusID", "Stylus ID: {0}"), StylusInfo->GetID())
				                    : FText();
		             });

		AddToOverlay(TopLeftOverlay,
		             [&StylusInfo = LastPacketData.StylusInfo]
		             {
			             return StylusInfo
				                    ? FText::Format(
					                    LOCTEXT("StylusName", "Stylus Name: {0}"),
					                    FText::FromString([&StylusInfo]
					                    {
						                    const FString Name = StylusInfo->GetName();
						                    return !Name.IsEmpty() ? Name : "<empty>";
					                    }()))
				                    : FText();
		             });

		AddToOverlay(TopLeftOverlay,
		             [&StylusInfo = LastPacketData.StylusInfo]
		             {
			             if (StylusInfo)
			             {
				             FString ButtonsString = "<none>";
				             for (int32 Index = 0, Num = StylusInfo->GetNumButtons(); Index < Num; ++Index)
				             {
					             if (const auto* Button = StylusInfo->GetButton(Index))
					             {
						             const FString Name = Button->GetName();
						             if (Index == 0)
						             {
							             ButtonsString = !Name.IsEmpty() ? Name : "<unknown>";
						             }
						             else
						             {
							             ButtonsString += ", " + (!Name.IsEmpty() ? Name : "<unknown>");
						             }
					             }
				             }
				             return FText::Format(LOCTEXT("StylusButtons", "Stylus Buttons: {0}"), FText::FromString(ButtonsString));
			             }
			             return FText();
		             });

		AddToOverlay(BottomLeftOverlay,
		             [&TabletContext = LastPacketData.TabletContext, &StylusInput = StylusInput, &EventHandlerThread = EventHandlerThread]
		             {
			             if (TabletContext && StylusInput)
			             {
				             return FText::Format(
					             LOCTEXT("PacketsPerSecond", "Packets Per Second: {0}"),
					             FMath::RoundToInt32(StylusInput->GetPacketsPerSecond(EventHandlerThread)));
			             }
			             return FText();
		             });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::TimerTick,
		                           [&TimerTick = LastPacketData.Packet.TimerTick]
		                           {
			                           return FText::Format(LOCTEXT("TimerTick", "Timer Tick: {0}"), TimerTick);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::SerialNumber,
		                           [&SerialNumber = LastPacketData.Packet.SerialNumber]
		                           {
			                           return FText::Format(LOCTEXT("SerialNumber", "Serial Number: {0}"), SerialNumber);
		                           });

		AddToOverlay(BottomLeftOverlay,
		             [&bIsSet = LastPacketData.bIsSet, &PenStatus = LastPacketData.Packet.PenStatus]
		             {
			             if (bIsSet)
			             {
				             FString PenStatusStr;
				             if (PenStatus == EPenStatus::None)
				             {
					             PenStatusStr = "None";
				             }
				             else
				             {
					             if ((PenStatus & EPenStatus::CursorIsTouching) != EPenStatus::None)
					             {
						             PenStatusStr += "CursorIsTouching";
					             }
					             if ((PenStatus & EPenStatus::CursorIsInverted) != EPenStatus::None)
					             {
						             PenStatusStr += PenStatusStr.IsEmpty() ? "CursorIsInverted" : " & CursorIsInverted";
					             }
					             if ((PenStatus & EPenStatus::NotUsed) != EPenStatus::None)
					             {
						             PenStatusStr += PenStatusStr.IsEmpty() ? "NotUsed" : " & NotUsed";
					             }
					             if ((PenStatus & EPenStatus::BarrelButtonPressed) != EPenStatus::None)
					             {
						             PenStatusStr += PenStatusStr.IsEmpty() ? "BarrelButtonPressed" : " & BarrelButtonPressed";
					             }
				             }

				             if (PenStatusStr.IsEmpty())
				             {
					             PenStatusStr = FString::Format(TEXT("Unknown ({0})"), {
						                                            static_cast<std::underlying_type_t<EPenStatus>>(PenStatus)
					                                            });
				             }

				             return FText::Format(LOCTEXT("PenStatus", "Pen Status: {0}"), FText::FromString(PenStatusStr));
			             }
			             return FText();
		             });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::X,
		                           [&X = LastPacketData.Packet.X]
		                           {
			                           return FText::Format(LOCTEXT("X", "X: {0}"), X);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::Y,
		                           [&Y = LastPacketData.Packet.Y]
		                           {
			                           return FText::Format(LOCTEXT("Y", "Y: {0}"), Y);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::Z,
		                           [&Z = LastPacketData.Packet.Z]
		                           {
			                           return FText::Format(LOCTEXT("Z", "Z: {0}"), Z);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::NormalPressure,
		                           [&NormalPressure = LastPacketData.Packet.NormalPressure]
		                           {
			                           return FText::Format(LOCTEXT("NormalPressure", "Normal Pressure: {0}"), NormalPressure);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::TangentPressure,
		                           [&TangentPressure = LastPacketData.Packet.TangentPressure]
		                           {
			                           return FText::Format(LOCTEXT("TangentPressure", "Tangent Pressure: {0}"), TangentPressure);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::ButtonPressure,
		                           [&ButtonPressure = LastPacketData.Packet.ButtonPressure]
		                           {
			                           return FText::Format(LOCTEXT("ButtonPressure", "Button Pressure: {0}"), ButtonPressure);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::XTiltOrientation,
		                           [&XTiltOrientation = LastPacketData.Packet.XTiltOrientation]
		                           {
			                           return FText::Format(LOCTEXT("XTiltOrientation", "X Tilt Orientation: {0}\u00B0"), XTiltOrientation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::YTiltOrientation,
		                           [&YTiltOrientation = LastPacketData.Packet.YTiltOrientation]
		                           {
			                           return FText::Format(LOCTEXT("YTiltOrientation", "Y Tilt Orientation: {0}\u00B0"), YTiltOrientation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::AzimuthOrientation,
		                           [&AzimuthOrientation = LastPacketData.Packet.AzimuthOrientation]
		                           {
			                           return FText::Format(LOCTEXT("AzimuthOrientation", "Azimuth Orientation: {0}\u00B0"), AzimuthOrientation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::AltitudeOrientation,
		                           [&AltitudeOrientation = LastPacketData.Packet.AltitudeOrientation]
		                           {
			                           return FText::Format(LOCTEXT("AltitudeOrientation", "Altitude Orientation: {0}\u00B0"), AltitudeOrientation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::TwistOrientation,
		                           [&TwistOrientation = LastPacketData.Packet.TwistOrientation]
		                           {
			                           return FText::Format(LOCTEXT("TwistOrientation", "Twist Orientation: {0}\u00B0"), TwistOrientation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::PitchRotation,
		                           [&PitchRotation = LastPacketData.Packet.PitchRotation]
		                           {
			                           return FText::Format(LOCTEXT("PitchRotation", "Pitch Rotation: {0}"), PitchRotation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::RollRotation,
		                           [&RollRotation = LastPacketData.Packet.RollRotation]
		                           {
			                           return FText::Format(LOCTEXT("RollRotation", "Roll Rotation: {0}"), RollRotation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::YawRotation,
		                           [&YawRotation = LastPacketData.Packet.YawRotation]
		                           {
			                           return FText::Format(LOCTEXT("YawRotation", "Yaw Rotation: {0}"), YawRotation);
		                           });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::Width,
								   [&Width = LastPacketData.Packet.Width]
								   {
									   return FText::Format(LOCTEXT("Width", "Width: {0}"), Width);
								   });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::Height,
								   [&Height = LastPacketData.Packet.Height]
								   {
									   return FText::Format(LOCTEXT("Height", "Height: {0}"), Height);
								   });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::FingerContactConfidence,
								   [&FingerContactConfidence = LastPacketData.Packet.FingerContactConfidence]
								   {
									   return FText::Format(LOCTEXT("FingerContactConfidence", "Finger Contact Confidence: {0}"), FingerContactConfidence);
								   });

		AddToOverlayWithVisibility(BottomLeftOverlay, ETabletSupportedProperties::DeviceContactID,
		                           [&DeviceContactID = LastPacketData.Packet.DeviceContactID]
		                           {
			                           return FText::Format(LOCTEXT("DeviceContactID", "Device Contact ID: {0}"), DeviceContactID);
		                           });
	}

	void SStylusInputDebugWidget::NotifyWidgetRelocated()
	{
		if (const TSharedPtr<SWindow> PrevWindow = StylusInputWindow.Pin())
		{
			if (const TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()))
			{
				if (PrevWindow.Get() == CurrentWindow.Get())
				{
					// Window did not change; no need to re-initialize stylus input.
					return;
				}
			}
		}

		UnregisterEventHandler();
		ReleaseStylusInput();

		AcquireStylusInput();
		RegisterEventHandler();
	}

	void SStylusInputDebugWidget::AcquireStylusInput()
	{
		if (StylusInput)
		{
			LogWarning("Stylus input instance has already been acquired.");
			return;
		}

		check(FSlateApplication::IsInitialized());
		const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if (!Window.IsValid())
		{
			LogError("Could not find widget window; stylus input instance has not been acquired.");
			return;
		}

		StylusInput = Interface == NAME_None ? CreateInstance(*Window) : CreateInstance(*Window, Interface, true);
		if (!StylusInput)
		{
			LogError("Could not acquire stylus input instance.");
			return;
		}

		StylusInputWindow = Window.ToWeakPtr();
	}

	void SStylusInputDebugWidget::ReleaseStylusInput()
	{
		if (EventHandler)
		{
			LogWarning("Event handler is still registered.");
		}

		if (StylusInput)
		{
			if (!ReleaseInstance(StylusInput))
			{
				LogError("Failed to release stylus input for StylusInput Debug Widget.");
			}

			StylusInput = nullptr;
		}

		StylusInputWindow = nullptr;
	}

	void SStylusInputDebugWidget::RegisterEventHandler()
	{
		if (!StylusInput)
		{
			LogWarning("Cannot register event handler since stylus input is unavailable.");
			return;
		}

		if (EventHandler)
		{
			LogWarning("Event handler is not null; please unregister event handler before trying to registering it again.");
		}

		if (EventHandlerThread == EEventHandlerThread::Asynchronous)
		{
			EventHandler = MakeUnique<FDebugEventHandlerAsynchronous>(FOnPacketCallback::CreateRaw(this, &SStylusInputDebugWidget::OnPacket),
			                                                          FOnDebugEventCallback::CreateRaw(this, &SStylusInputDebugWidget::OnDebugEvent));
		}
		else
		{
			EventHandler = MakeUnique<FDebugEventHandlerOnGameThread>(FOnPacketCallback::CreateRaw(this, &SStylusInputDebugWidget::OnPacket),
			                                                          FOnDebugEventCallback::CreateRaw(this, &SStylusInputDebugWidget::OnDebugEvent));
		}

		if (StylusInput->AddEventHandler(EventHandler.Get(), EventHandlerThread))
		{
			LogVerbose(EventHandlerThread == EEventHandlerThread::Asynchronous
				           ? "Registered event handler on asynchronous thread."
				           : "Registered event handler on game thread.");
		}
		else
		{
			LogError("Failed to register event handler.");
		}
	}

	void SStylusInputDebugWidget::UnregisterEventHandler()
	{
		if (!EventHandler)
		{
			LogWarning("Cannot unregister event handler since it is invalid.");
			return;
		}

		if (StylusInput)
		{
			if (StylusInput->RemoveEventHandler(EventHandler.Get()))
			{
				LogVerbose("Unregistered event handler for StylusInput Debug Widget.");
			}
			else
			{
				LogError("Failed to unregister event handler for StylusInput Debug Widget.");
			}
		}
		else
		{
			LogWarning("Cannot unregister event handler since stylus input is unavailable.");
		}

		EventHandler.Reset();
	}

	FDebugEventHandlerAsynchronous::FDebugEventHandlerAsynchronous(FOnPacketCallback&& OnPacketCallback, FOnDebugEventCallback&& OnDebugEventCallback)
		: OnPacketCallback(MoveTemp(OnPacketCallback)), OnDebugEventCallback(MoveTemp(OnDebugEventCallback))
	{
		check(this->OnPacketCallback.IsBound());
		check(this->OnDebugEventCallback.IsBound());
	}

	void FDebugEventHandlerAsynchronous::OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance*)
	{
		PacketQueue.Enqueue(Packet);
	}

	void FDebugEventHandlerAsynchronous::OnDebugEvent(const FString& Message, IStylusInputInstance*)
	{
		DebugEventQueue.Enqueue(Message);
	}

	void FDebugEventHandlerAsynchronous::Tick(float DeltaTime)
	{
		FStylusInputPacket Packet;
		while (PacketQueue.Dequeue(Packet))
		{
			OnPacketCallback.Execute(Packet);
		}

		FString Message;
		while (DebugEventQueue.Dequeue(Message))
		{
			OnDebugEventCallback.Execute(Message);
		}
	}

	FDebugEventHandlerOnGameThread::FDebugEventHandlerOnGameThread(FOnPacketCallback&& OnPacketCallback, FOnDebugEventCallback&& OnDebugEventCallback)
		: OnPacketCallback(MoveTemp(OnPacketCallback)), OnDebugEventCallback(MoveTemp(OnDebugEventCallback))
	{
		check(this->OnPacketCallback.IsBound());
		check(this->OnDebugEventCallback.IsBound());
	}

	void FDebugEventHandlerOnGameThread::OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance*)
	{
		OnPacketCallback.Execute(Packet);
	}

	void FDebugEventHandlerOnGameThread::OnDebugEvent(const FString& Message, IStylusInputInstance*)
	{
		OnDebugEventCallback.Execute(Message);
	}

	SStylusInputDebugWidget::~SStylusInputDebugWidget()
	{
		UnregisterEventHandler();
		ReleaseStylusInput();
	}

	void SStylusInputDebugWidget::OnPacket(const FStylusInputPacket& Packet)
	{
		LastPacketData.bIsSet = true;
		LastPacketData.Packet = Packet;

		if (!LastPacketData.TabletContext || (LastPacketData.TabletContext && LastPacketData.TabletContext->GetID() != Packet.TabletContextID))
		{
			LastPacketData.TabletContext = GetTabletContext(Packet.TabletContextID);
		}

		if (!LastPacketData.StylusInfo || (LastPacketData.StylusInfo && LastPacketData.StylusInfo->GetID() != Packet.CursorID))
		{
			LastPacketData.StylusInfo = GetStylusInfo(Packet.CursorID);
		}

		if (PaintWidget)
		{
			PaintWidget->Add(Packet);
		}
	}

	void SStylusInputDebugWidget::OnDebugEvent(const FString& Message)
	{
		if (DebugMessages.IsEmpty())
		{
			DebugMessages = Message;
		}
		else
		{
			DebugMessages += "\n" + Message;
		}
	}

	TSharedRef<SWidget> SStylusInputDebugWidget::GetInterfaceMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		for (const FName& AvailableInterface : GetAvailableInterfaces())
		{
			MenuBuilder.AddMenuEntry(FText::FromName(AvailableInterface),
			                         {},
			                         FSlateIcon(),
			                         FUIAction(FExecuteAction::CreateLambda([this, AvailableInterface]
			                                   {
				                                   SetInterface(AvailableInterface);
			                                   }),
			                                   FCanExecuteAction(),
			                                   FIsActionChecked::CreateLambda([&Interface = Interface, AvailableInterface]
			                                   {
				                                   return Interface == AvailableInterface;
			                                   })),
			                         NAME_None,
			                         EUserInterfaceActionType::RadioButton);
		}

		return MenuBuilder.MakeWidget();
	}

	void SStylusInputDebugWidget::SetInterface(FName InInterface)
	{
		if (Interface != InInterface)
		{
			UnregisterEventHandler();
			ReleaseStylusInput();

			Interface = InInterface;
			LastPacketData = {};

			AcquireStylusInput();
			RegisterEventHandler();
		}
	}

	TSharedRef<SWidget> SStylusInputDebugWidget::GetEventHandlerThreadMenu()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.AddMenuEntry(LOCTEXT("Asynchronous", "Asynchronous"),
		                         LOCTEXT("AsynchronousTooltip", "Event handler is evaluated on a dedicated thread"),
		                         FSlateIcon(),
		                         FUIAction(FExecuteAction::CreateLambda([this]
		                                   {
			                                   SetEventHandlerThread(EEventHandlerThread::Asynchronous);
		                                   }),
		                                   FCanExecuteAction(),
		                                   FIsActionChecked::CreateLambda([&EventHandlerThread = EventHandlerThread]
		                                   {
			                                   return EventHandlerThread == EEventHandlerThread::Asynchronous;
		                                   })),
		                         NAME_None,
		                         EUserInterfaceActionType::RadioButton);

		MenuBuilder.AddMenuEntry(LOCTEXT("GameThread", "On Game Thread"),
		                         LOCTEXT("GameThreadTooltip", "Event handler is evaluated on the game thread"),
		                         FSlateIcon(),
		                         FUIAction(FExecuteAction::CreateLambda([this]
		                                   {
			                                   SetEventHandlerThread(EEventHandlerThread::OnGameThread);
		                                   }),
		                                   FCanExecuteAction(),
		                                   FIsActionChecked::CreateLambda([&EventHandlerThread = EventHandlerThread]
		                                   {
			                                   return EventHandlerThread == EEventHandlerThread::OnGameThread;
		                                   })),
		                         NAME_None,
		                         EUserInterfaceActionType::RadioButton);

		return MenuBuilder.MakeWidget();
	}

	void SStylusInputDebugWidget::SetEventHandlerThread(EEventHandlerThread InEventHandlerThread)
	{
		if (EventHandlerThread != InEventHandlerThread)
		{
			UnregisterEventHandler();

			EventHandlerThread = InEventHandlerThread;

			RegisterEventHandler();
		}
	}

	const IStylusInputTabletContext* SStylusInputDebugWidget::GetTabletContext(uint32 TabletContextID)
	{
		if (!StylusInput)
		{
			return nullptr;
		}

		const TSharedPtr<IStylusInputTabletContext>* TabletContext = TabletContexts.Find(TabletContextID);
		if (!TabletContext)
		{
			if (const TSharedPtr<IStylusInputTabletContext>& NewTabletContext = StylusInput->GetTabletContext(TabletContextID))
			{
				TabletContext = &TabletContexts.Emplace(TabletContextID, NewTabletContext);
			}
		}

		return TabletContext ? TabletContext->Get() : nullptr;
	}

	const IStylusInputStylusInfo* SStylusInputDebugWidget::GetStylusInfo(uint32 StylusID)
	{
		if (!StylusInput)
		{
			return nullptr;
		}

		const TSharedPtr<IStylusInputStylusInfo>* StylusInfo = StylusInfos.Find(StylusID);
		if (!StylusInfo)
		{
			if (const TSharedPtr<IStylusInputStylusInfo> NewStylusInfo = StylusInput->GetStylusInfo(StylusID))
			{
				StylusInfo = &StylusInfos.Emplace(StylusID, NewStylusInfo);
			}
			else
			{
				return nullptr;
			}
		}

		return StylusInfo->Get();
	}
}

#undef LOCTEXT_NAMESPACE
