// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <Widgets/SCompoundWidget.h>

#include "StylusInputDebugPaintWidget.h"
#include "TickableEditorObject.h"
#include "Containers/SpscQueue.h"

namespace UE::StylusInput::DebugWidget
{
	DECLARE_DELEGATE_OneParam(FOnPacketCallback, const FStylusInputPacket&);
	DECLARE_DELEGATE_OneParam(FOnDebugEventCallback, const FString&);

	class FDebugEventHandlerAsynchronous final : public IStylusInputEventHandler, FTickableEditorObject
	{
	public:
		FDebugEventHandlerAsynchronous(FOnPacketCallback&& OnPacketCallback, FOnDebugEventCallback&& OnDebugEventCallback);

		virtual FString GetName() override { return "DebugEventHandlerAsynchronous"; }

		virtual void OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance* Instance) override;
		virtual void OnDebugEvent(const FString& Message, IStylusInputInstance* Instance) override;

		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(StylusInput_DebugEventHandlerAsynchronous, STATGROUP_Tickables); }

	private:
		FOnPacketCallback OnPacketCallback;
		FOnDebugEventCallback OnDebugEventCallback;

		TSpscQueue<FStylusInputPacket> PacketQueue;
		TSpscQueue<FString> DebugEventQueue;
	};

	class FDebugEventHandlerOnGameThread final : public IStylusInputEventHandler
	{
	public:
		FDebugEventHandlerOnGameThread(FOnPacketCallback&& OnPacketCallback, FOnDebugEventCallback&& OnDebugEventCallback);

		virtual FString GetName() override { return "DebugEventHandlerOnGameThread"; }

		virtual void OnPacket(const FStylusInputPacket& Packet, IStylusInputInstance* Instance) override;
		virtual void OnDebugEvent(const FString& Message, IStylusInputInstance* Instance) override;

	private:
		FOnPacketCallback OnPacketCallback;
		FOnDebugEventCallback OnDebugEventCallback;
	};

	class SStylusInputDebugWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SStylusInputDebugWidget)
			{
			}
		SLATE_END_ARGS()

		virtual ~SStylusInputDebugWidget() override;

		void Construct(const FArguments& Args);

		void NotifyWidgetRelocated();

	private:
		void AcquireStylusInput();
		void ReleaseStylusInput();
		void RegisterEventHandler();
		void UnregisterEventHandler();

		const IStylusInputTabletContext* GetTabletContext(uint32 TabletContextID);
		const IStylusInputStylusInfo* GetStylusInfo(uint32 StylusID);

		void OnPacket(const FStylusInputPacket& Packet);
		void OnDebugEvent(const FString& Message);

		TSharedRef<SWidget> GetInterfaceMenu();
		void SetInterface(FName InInterface);
		FName Interface;

		TSharedRef<SWidget> GetEventHandlerThreadMenu();
		void SetEventHandlerThread(EEventHandlerThread InEventHandlerThread);
		EEventHandlerThread EventHandlerThread = EEventHandlerThread::OnGameThread;

		IStylusInputInstance* StylusInput = nullptr;
		TUniquePtr<IStylusInputEventHandler> EventHandler;
		TWeakPtr<SWindow> StylusInputWindow;

		TSharedPtr<SStylusInputDebugPaintWidget> PaintWidget;

		TMap<uint32, TSharedPtr<IStylusInputTabletContext>> TabletContexts;
		TMap<uint32, TSharedPtr<IStylusInputStylusInfo>> StylusInfos;

		FString DebugMessages;

		struct FLastPacketData
		{
			bool bIsSet = false;
			FStylusInputPacket Packet;
			const IStylusInputTabletContext* TabletContext = nullptr;
			const IStylusInputStylusInfo* StylusInfo = nullptr;
		};

		FLastPacketData LastPacketData;
	};
}
