// Copyright Epic Games, Inc. All Rights Reserved.

#include <WorkspaceMenuStructure.h>
#include <WorkspaceMenuStructureModule.h>
#include <Modules/ModuleManager.h>
#include <Styling/SlateStyle.h>
#include <Styling/SlateStyleRegistry.h>
#include <Widgets/Docking/SDockTab.h>

#include "StylusInputDebugWidget.h"

#define LOCTEXT_NAMESPACE "StylusInputDebugWidgetModule"

namespace UE::StylusInput::DebugWidget
{
	class FStylusInputDebugWidgetModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		void RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup);
		void UnregisterTabSpawners();

	private:
		TSharedRef<SDockTab> MakeDebugWidgetTab(const FSpawnTabArgs&);
		TSharedRef<SWidget> GetDebugWidget();
		void SetupStyleSet();
		void ResetStyleSet();

		bool bHasRegisteredTabSpawners = false;
		TWeakPtr<SStylusInputDebugWidget> DebugWidgetPtr;
		TSharedPtr<FSlateStyleSet> StyleSet;
	};

	void FStylusInputDebugWidgetModule::StartupModule()
	{
		SetupStyleSet();
		RegisterTabSpawners(nullptr);
	}

	void FStylusInputDebugWidgetModule::ShutdownModule()
	{
		UnregisterTabSpawners();
		ResetStyleSet();
	}

	void FStylusInputDebugWidgetModule::RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
	{
		if (bHasRegisteredTabSpawners)
		{
			UnregisterTabSpawners();
		}

		FTabSpawnerEntry& DebugWidgetSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("StylusInputDebugWidget",
			FOnSpawnTab::CreateRaw(this, &FStylusInputDebugWidgetModule::MakeDebugWidgetTab))
				.SetDisplayName(LOCTEXT("DebugWidgetTitle", "Stylus Input"))
				.SetTooltipText(LOCTEXT("DebugWidgetTooltip", "Open a debug widget to verify stylus input event handling."))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
				.SetIcon(FSlateIcon(StyleSet->GetStyleSetName(), "StylusInput.Small"));

		if (WorkspaceGroup.IsValid())
		{
			DebugWidgetSpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}

		bHasRegisteredTabSpawners = true;
	}

	void FStylusInputDebugWidgetModule::UnregisterTabSpawners()
	{
		bHasRegisteredTabSpawners = false;

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("StylusInputDebugWidget");
	}

	TSharedRef<SDockTab> FStylusInputDebugWidgetModule::MakeDebugWidgetTab(const FSpawnTabArgs&)
	{
		TSharedRef<SDockTab> Tab = SNew(SDockTab).TabRole(NomadTab);
		Tab->SetContent(GetDebugWidget());

		// Relocating the tab might mean that it is parented under a new window, which would require re-initializing the stylus input instance.
		Tab->SetOnTabRelocated(FSimpleDelegate::CreateLambda([DebugWidgetPtr = DebugWidgetPtr]
		{
			if (const TSharedPtr<SStylusInputDebugWidget> DebugWidget = DebugWidgetPtr.Pin())
			{
				DebugWidget->NotifyWidgetRelocated();
			}
		}));

		return Tab;
	}

	TSharedRef<SWidget> FStylusInputDebugWidgetModule::GetDebugWidget()
	{
		TSharedPtr<SStylusInputDebugWidget> DebugWidget = DebugWidgetPtr.Pin();

		if (!DebugWidget.IsValid())
		{
			DebugWidget = SNew(SStylusInputDebugWidget);
			DebugWidgetPtr = DebugWidget;
		}

		return DebugWidget.ToSharedRef();
	}

	void FStylusInputDebugWidgetModule::SetupStyleSet()
	{
		static FName StylusInputDebugWidgetStyle(TEXT("StylusInputDebugWidgetStyle"));
		StyleSet = MakeShared<FSlateStyleSet>(StylusInputDebugWidgetStyle);

		StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/StylusInput/Resources"));

		StyleSet->Set("StylusInput.Small",
		              new FSlateVectorImageBrush(StyleSet->RootToContentDir(TEXT("StylusInput_16"), TEXT(".svg")), {16.0f, 16.0f}));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}

	void FStylusInputDebugWidgetModule::ResetStyleSet()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
		StyleSet.Reset();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::StylusInput::DebugWidget::FStylusInputDebugWidgetModule, StylusInputDebugWidget);
