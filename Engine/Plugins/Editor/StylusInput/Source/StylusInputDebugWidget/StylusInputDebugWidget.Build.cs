// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StylusInputDebugWidget : ModuleRules
	{
		public StylusInputDebugWidget(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				[
					"Core",
					"Engine",
					"Slate",
					"SlateCore",
					"StylusInput",
					"UnrealEd",
					"WorkspaceMenuStructure"
				]
			);
		}
	}
}
