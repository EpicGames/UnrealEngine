// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacInstance.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>
#include <Algo/Transform.h>

#include "MacInterface.h"
#include "NSEventHandler.h"
#include "MacTabletContext.h"

#define LOCTEXT_NAMESPACE "MacInstance"
#define LOG_PREAMBLE "MacInstance"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Mac
{
	FMacInstance::FMacInstance(uint32 ID, const FCocoaWindow* OSWindowHandle)
		: CocoaWindow(OSWindowHandle)
		, ID(ID)
	{
	}

	FMacInstance::~FMacInstance()
	{
	}

	bool FMacInstance::AddEventHandler(IStylusInputEventHandler* EventHandler, const EEventHandlerThread Thread)
	{
		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to add nullptr as event handler.");
			return false;
		}
		
		if (!NSEventHandler)
		{
			EnableEventHandler(EEventHandlerThread::OnGameThread, EventHandler);
			
			if (!NSEventHandler)
			{
				LogError(LOG_PREAMBLE, FString::Format(
						 TEXT("Event handler '{0}' was not added since NSEventHandler could not be installed."),
						 {EventHandler->GetName()}));
				return false;
			}
			return true;
		}
		
		return NSEventHandler->AddEventHandler(EventHandler);
	}

	bool FMacInstance::RemoveEventHandler(IStylusInputEventHandler* EventHandler)
	{
		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to remove nullptr event handler.");
			return false;
		}

		bool bWasRemoved = false;

		if (NSEventHandler && NSEventHandler->RemoveEventHandler(EventHandler))
		{
			bWasRemoved = true;

			if (NSEventHandler->NumEventHandlers() == 0)
			{
				DisableEventHandler(EEventHandlerThread::OnGameThread);
			}
		}

		if (!bWasRemoved)
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Event handler '%s' does not exist."), {EventHandler->GetName()}));
		}

		return bWasRemoved;
	}

	const TSharedPtr<IStylusInputTabletContext> FMacInstance::GetTabletContext(const uint32 TabletContextID)
	{
		return TabletContexts.Get(TabletContextID);
	}

	const TSharedPtr<IStylusInputStylusInfo> FMacInstance::GetStylusInfo(uint32 StylusID)
	{
		return StylusInfo.Get(StylusID);
	}

	float FMacInstance::GetPacketsPerSecond(const EEventHandlerThread EventHandlerThread) const
	{
		return NSEventHandler ? NSEventHandler->GetPacketsPerSecond() : 0.0f;
	}

	void FMacInstance::SetupTabletContexts()
	{
		if( NSEventHandler )
		{
			GetMacTabletDevices(NSEventHandler->GetTabletContexts());
			for (int i = 0; i < NSEventHandler->GetTabletContexts().Num(); i++)
			{
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Added tablet context for device: %s, productID: %d, vendorID: %d"), {*NSEventHandler->GetTabletContexts()[i]->ProductName, NSEventHandler->GetTabletContexts()[i]->ProductID, NSEventHandler->GetTabletContexts()[i]->VendorID}));
			}
			UpdateTabletContexts(NSEventHandler->GetTabletContexts());
		}
	}

	void FMacInstance::UpdateTabletContexts(FTabletContextContainer& InTabletContexts)
	{
		TabletContexts.Update(InTabletContexts);
	}

	void FMacInstance::EnableEventHandler(const EEventHandlerThread EventHandlerThread, IStylusInputEventHandler* EventHandler)
	{
		NSEventHandler = MakeUnique<FNSEventHandler>(
			this,
			CocoaWindow,
			EventHandler
		);
		
		SetupTabletContexts();
		NSEventHandler->StartListen();
	}

	void FMacInstance::DisableEventHandler(const EEventHandlerThread EventHandlerThread)
	{
		if( NSEventHandler )
			NSEventHandler->StopListen();
		NSEventHandler.Reset();
	}

	FName FMacInstance::GetInterfaceName()
	{
		static FName Name("NSEvent");
		return Name;
	}

	FText FMacInstance::GetName()
	{
		return FText::Format(LOCTEXT("NSEvent", "NSEvent #{0}"), ID);
	}

	bool FMacInstance::WasInitializedSuccessfully() const
	{
		return true;
	}
}

#undef LOG_PREAMBLE
#undef LOCTEXT_NAMESPACE
