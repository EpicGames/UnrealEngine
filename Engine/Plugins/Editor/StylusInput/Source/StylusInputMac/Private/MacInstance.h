// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <Containers/Array.h>
#include <Templates/UniquePtr.h>

#include "MacTabletContext.h"
#include "Mac/MacApplication.h"
#include "Mac/MacPlatformApplicationMisc.h"

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
}

namespace UE::StylusInput::Mac
{
	class FNSEventHandler;

	class FMacInstance : public IStylusInputInstance
	{
	public:
		explicit FMacInstance(uint32 ID, const FCocoaWindow* OSWindowHandle);
		virtual ~FMacInstance() override;

		virtual bool AddEventHandler(IStylusInputEventHandler* EventHandler, EEventHandlerThread Thread) override;
		virtual bool RemoveEventHandler(IStylusInputEventHandler* EventHandler) override;

		virtual const TSharedPtr<IStylusInputTabletContext> GetTabletContext(uint32 TabletContextID) override;
		virtual const TSharedPtr<IStylusInputStylusInfo> GetStylusInfo(uint32 StylusID) override;

		virtual float GetPacketsPerSecond(EEventHandlerThread EventHandlerThread) const override;

		virtual FName GetInterfaceName() override;
		virtual FText GetName() override;

		virtual bool WasInitializedSuccessfully() const override;

	private:
		void EnableEventHandler(EEventHandlerThread EventHandlerThread, IStylusInputEventHandler* EventHandler);
		void DisableEventHandler(EEventHandlerThread EventHandlerThread);

		// There isn't such a thing as a "Tablet context" with NSEvent. Instead, we'll store all the devices known by the user computer and sort the events by ID
		void SetupTabletContexts();

		void UpdateTabletContexts(FTabletContextContainer& InTabletContexts);

		const FCocoaWindow* CocoaWindow;

		TUniquePtr<FNSEventHandler> NSEventHandler;

		FTabletContextContainer TabletContexts;
		FStylusInfoContainer StylusInfo;

		const uint32 ID;
	};
}
