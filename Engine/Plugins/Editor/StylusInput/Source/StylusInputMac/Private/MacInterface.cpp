// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacInterface.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>

#include <GenericPlatform/GenericWindow.h>
#include <Templates/SharedPointer.h>
#include <Widgets/SWindow.h>

#include "MacInstance.h"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Mac
{
	FName FMacInterface::GetName() const
	{
		static FName Name("NSEvent");
		return Name;
	}

	IStylusInputInstance* FMacInterface::CreateInstance(SWindow& Window)
	{
		if (FRefCountedInstance* ExistingRefCountedInstance = Instances.Find(&Window))
		{
			++ExistingRefCountedInstance->RefCount;
			return ExistingRefCountedInstance->Instance.Get();
		}

		FCocoaWindow* OSWindowHandle = [&Window]
		{
			const TSharedPtr<const FGenericWindow> NativeWindow = Window.GetNativeWindow();
			return NativeWindow.IsValid() ? static_cast<FCocoaWindow*>(NativeWindow->GetOSWindowHandle()) : nullptr;
		}();

		if (!OSWindowHandle)
		{
			LogError("MacInterface", "Could not get native window handle.");
			return nullptr;
		}

		FMacInstance* NewInstance = Instances.Emplace(
			&Window, {MakeUnique<FMacInstance>(NextInstanceID++, OSWindowHandle), 1}).Instance.Get();
		if (!ensureMsgf(NewInstance, TEXT("MacInterface: Failed to create stylus input instance.")))
		{
			Instances.Remove(&Window);
			return nullptr;
		}

		return NewInstance;
	}

	bool FMacInterface::ReleaseInstance(IStylusInputInstance* Instance)
	{
		check(Instance);

		FMacInstance *const MacInstance = static_cast<FMacInstance*>(Instance);

		// Find existing instance
		for (TTuple<SWindow*, FRefCountedInstance>& Entry : Instances)
		{
			FRefCountedInstance& RefCountedInstance = Entry.Get<1>();

			if (RefCountedInstance.Instance.Get() == MacInstance)
			{
				// Decrease reference count
				check(RefCountedInstance.RefCount > 0);
				if (--RefCountedInstance.RefCount == 0)
				{
					// Delete if there are no references left
					Instances.Remove(Entry.Key);
				}

				return true;
			}
		}

		ensureMsgf(false, TEXT("MacInterface: Failed to find provided instance."));
		return false;
	}

	TUniquePtr<IStylusInputInterface> FMacInterface::Create()
	{
		TUniquePtr<FMacInterface> MacImpl = MakeUnique<FMacInterface>();
		return TUniquePtr<IStylusInputInterface>(MoveTemp(MacImpl));
	}
}
