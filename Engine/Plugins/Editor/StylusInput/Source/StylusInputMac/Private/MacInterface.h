// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputInterface.h>
#include <Containers/Array.h>
#include <Containers/Map.h>
#include <Templates/UniquePtr.h>

namespace UE::StylusInput::Private
{
	class FStylusInputImpl;
}

namespace UE::StylusInput::Mac
{
	class FMacInstance;

	class FMacInterface : public IStylusInputInterface
	{
	public:
		virtual FName GetName() const override;

		virtual IStylusInputInstance* CreateInstance(SWindow& Window) override;
		virtual bool ReleaseInstance(IStylusInputInstance* Instance) override;

	private:
		friend class FStylusInputMacModule;

		static TUniquePtr<IStylusInputInterface> Create();

		struct FRefCountedInstance
		{
			TUniquePtr<FMacInstance> Instance;
			int32 RefCount;
		};

		TMap<SWindow*, FRefCountedInstance> Instances;
		uint32 NextInstanceID = 0;
	};
}
