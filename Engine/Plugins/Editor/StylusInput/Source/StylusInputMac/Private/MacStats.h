// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Mac/MacPlatformTime.h>

namespace UE::StylusInput::Mac
{
	class FPacketStats
	{
	public:
		void NewPacket()
		{
			const uint64 Timestamp = FPlatformTime::Cycles64();

			if (LatestTimestamp - EarliestTimestamp >= CyclesPerSecond)
			{
				PacketsPerSecond = EarliestTimestamp != LatestTimestamp ? NumPackets / ((LatestTimestamp - EarliestTimestamp) / CyclesPerSecond) : 0.0f;
				EarliestTimestamp = Timestamp;
				NumPackets = -1;
			}

			LatestTimestamp = Timestamp;
			++NumPackets;
		}

		float GetPacketsPerSecond() const
		{
			const uint64 Timestamp = FPlatformTime::Cycles64();
			return Timestamp - LatestTimestamp <= CyclesPerSecond ? PacketsPerSecond : 0.0f;
		}

	private:
		uint64 EarliestTimestamp = 0;
		uint64 LatestTimestamp = 0;
		uint32 NumPackets = 0;
		float PacketsPerSecond = 0.0f;
		const double CyclesPerSecond = 1.0 / FPlatformTime::GetSecondsPerCycle64();
	};
}
