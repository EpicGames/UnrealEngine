// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacTabletContext.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <IOKit/hid/IOHIDManager.h>
#import <Foundation/Foundation.h>

#define LOG_PREAMBLE "MacTabletContext"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Mac
{
	static FString CFStringToFString(CFStringRef cfStr)
	{
		if (!cfStr) return FString();
		NSString* nsStr = (__bridge NSString*)cfStr;
		return FString(nsStr);
	}

	bool GetMacTabletDevices(FTabletContextContainer& OutDevices)
	{
		OutDevices.Clear();

		IOHIDManagerRef HidManager = IOHIDManagerCreate(kCFAllocatorDefault, kIOHIDOptionsTypeNone);
		if (!HidManager)
		{
			return false;
		}

		// Match Generic Desktop devices (mouse and such)
		NSDictionary* matchingGenericDesktopDict = @{
			@(kIOHIDDeviceUsagePageKey): @(kHIDPage_GenericDesktop)
		};

		// Match Digitizer devices (tablets)
		NSDictionary* matchingDigitizerDict = @{
			@(kIOHIDDeviceUsagePageKey): @(kHIDPage_Digitizer)
		};

		NSArray* matchingArray = @[matchingGenericDesktopDict, matchingDigitizerDict];

		IOHIDManagerSetDeviceMatchingMultiple(HidManager, (__bridge CFArrayRef) matchingArray);
		IOHIDManagerScheduleWithRunLoop(HidManager, CFRunLoopGetCurrent(), kCFRunLoopDefaultMode);
		IOHIDManagerOpen(HidManager, kIOHIDOptionsTypeNone);

		CFSetRef DeviceSet = IOHIDManagerCopyDevices(HidManager);
		if (!DeviceSet || CFSetGetCount(DeviceSet) == 0)
		{
			if (DeviceSet) 
				CFRelease(DeviceSet);
			CFRelease(HidManager);
			return false;
		}

		NSSet* Devices = (__bridge NSSet*)DeviceSet;

		for (id DeviceObj in Devices)
		{
			IOHIDDeviceRef Device = (__bridge IOHIDDeviceRef)DeviceObj;

			//---
			CFStringRef productKey = (CFStringRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDProductKey));
			NSNumber* deviceUsagePageNumber = (__bridge NSNumber*)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDPrimaryUsagePageKey));
			NSNumber* deviceUsageNumber = (__bridge NSNumber*)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDPrimaryUsageKey));
			uint32_t deviceUsagePage = deviceUsagePageNumber.unsignedIntValue;
			uint32_t deviceUsage = deviceUsageNumber.unsignedIntValue;

			NSLog(@"Supported Device: %@ | UsagePage: %@, Usage: %@", productKey, deviceUsagePageNumber, deviceUsageNumber);
			//---

			CFStringRef product = (CFStringRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDProductKey));
			CFStringRef vendor = (CFStringRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDManufacturerKey));
			CFStringRef serial = (CFStringRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDSerialNumberKey));
			CFNumberRef vendorID = (CFNumberRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDVendorIDKey));
			CFNumberRef productID = (CFNumberRef)IOHIDDeviceGetProperty(Device, CFSTR(kIOHIDProductIDKey));

			NSLog(@"Supported product: %@ | vendor: %@, serial: %@, vendorID: %@, productID: %@", product, vendor, serial, vendorID, productID);

			uint32 ID = 0;
			if(productID) //productID is 0 if and only if we're dealing with the internal pointing device (touchpad)
				CFNumberGetValue(productID, kCFNumberSInt32Type, &ID);

			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("ID: %d"), {ID}));

			TSharedRef<FTabletContext> context = OutDevices.Add(ID);

			context->ProductName = product;
			context->ProductID = ID;

			context->VendorName  = vendor;
			if (vendorID)
				CFNumberGetValue(vendorID, kCFNumberSInt32Type, &context->VendorID);

			context->SerialNumber = serial;

			CFArrayRef elements = IOHIDDeviceCopyMatchingElements(Device, NULL, kIOHIDOptionsTypeNone);
			if (elements) 
			{
				for (CFIndex i = 0; i < CFArrayGetCount(elements); i++) 
				{
					IOHIDElementRef element = (IOHIDElementRef)CFArrayGetValueAtIndex(elements, i);
					uint32_t elementUsagePage = IOHIDElementGetUsagePage(element);
					uint32_t elementUsage = IOHIDElementGetUsage(element);
					NSLog(@"Supported element usage page 0x%X and usage: 0x%X", elementUsagePage, elementUsage);

					if( elementUsagePage == kHIDPage_GenericDesktop )
					{
						switch(elementUsage)
						{
							case kHIDUsage_GD_Mouse:
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::Integrated;

								NSLog(@"Supported Integrated");
								break;
							case kHIDUsage_GD_X:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::X;
								context->InputRectangle.Min[0] = IOHIDElementGetLogicalMin(element);
								context->InputRectangle.Max[0] = IOHIDElementGetLogicalMax(element);
								NSLog(@"Supported X");
								break;
							case kHIDUsage_GD_Y:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::Y;
								context->InputRectangle.Min[1] = IOHIDElementGetLogicalMin(element);
								context->InputRectangle.Max[1] = IOHIDElementGetLogicalMax(element);
								NSLog(@"Supported Y");
								break;
							case kHIDUsage_GD_Z:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::Z;
								NSLog(@"Supported Z");
								break;
						}
					}
					if( elementUsagePage == kHIDPage_Digitizer || elementUsagePage >= 0xFF00 ) //Vendor-defined enum field
					{
						switch(elementUsage)
						{
							case kHIDUsage_Dig_Pen:
							case kHIDUsage_Dig_LightPen:
							case kHIDUsage_Dig_Stylus:
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::CursorMustTouch;
								NSLog(@"CursorMustTouch Supported");
								break;
							case kHIDUsage_Dig_TouchScreen:
							case kHIDUsage_Dig_TouchPad:
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::Integrated;
								NSLog(@"Dig Integrated Supported");
								break;
							case kHIDUsage_Dig_InRange:
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::HardProximity;
								NSLog(@"Dig HardProximity Supported");
								break;
							case kHIDUsage_Dig_BarrelSwitch:
							case kHIDUsage_Dig_Eraser:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::PacketStatus;
								NSLog(@"PacketStatus Supported");
								break;
							case kHIDUsage_Dig_Touch:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::PacketStatus;
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::Integrated;
								NSLog(@"Touch Supported");
								break;
							case kHIDUsage_Dig_TipPressure:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::NormalPressure;
								NSLog(@"Pressure Supported");
								break;
							case kHIDUsage_Dig_BarrelPressure:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::TangentPressure;
								NSLog(@"Tangent Pressure Supported");
								break;
							case kHIDUsage_Dig_XTilt:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::XTiltOrientation;
								NSLog(@"XTilt Supported");
								break;
							case kHIDUsage_Dig_YTilt:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::YTiltOrientation;
								NSLog(@"YTilt Supported");
								break;
							case kHIDUsage_Dig_Azimuth:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::AzimuthOrientation;
								NSLog(@"Azimuth Supported");
								break;
							case kHIDUsage_Dig_Altitude:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::AltitudeOrientation;
								NSLog(@"Altitude Supported");
								break;
							case kHIDUsage_Dig_Twist:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::TwistOrientation;
								NSLog(@"Twist Supported");
								break;
							case kHIDUsage_Dig_Width:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::Width;
								NSLog(@"Width Supported");
								break;
							case kHIDUsage_Dig_Height:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::Height;
								NSLog(@"Height Supported");
								break;
							case kHIDUsage_Dig_Finger:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::FingerContactConfidence;
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::Integrated;
								NSLog(@"Finger Confidence Supported");
								break;
							case kHIDUsage_Dig_ContactIdentifier:
								context->SupportedProperties = context->SupportedProperties | ETabletSupportedProperties::DeviceContactID;
								context->HardwareCapabilities = context->HardwareCapabilities | ETabletHardwareCapabilities::CursorsHavePhysicalIds;
								NSLog(@"Device contact ID Supported");
								break;
						}
					}
				}
				CFRelease(elements);
			}
		}
		CFRelease(DeviceSet);
		CFRelease(HidManager);

		return true;
	}
}

#undef LOG_PREAMBLE
