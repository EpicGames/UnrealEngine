// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputPacket.h>
#include <StylusInputTabletContext.h>
#include <StylusInputUtils.h>
#include <Math/IntRect.h>
#include <Math/Vector.h>

namespace UE::StylusInput::Mac
{
	enum class EPacketPropertyType : int8
	{
		X = 0, // The x-coordinate in the tablet coordinate space. Each packet contains this property by default. The origin (0,0) of the tablet is the upper-left corner.
		Y, // The y-coordinate in the tablet coordinate space. Each packet contains this property by default. The origin (0,0) of the tablet is the upper-left corner.
		Z, // The z-coordinate or distance of the pen tip from the tablet surface. The TabletPropertyMetricUnit enumeration type determines the unit of measurement for this property.
		PacketStatus, // Contains one or more of the following flag values: The cursor is touching the drawing surface (Value = 1). The cursor is inverted. For example, the eraser end of the pen is pointing toward the surface (Value = 2). Not used (Value = 4). The barrel button is pressed (Value = 8).
		TimerTick, // The time the packet was generated.
		SerialNumber, // The packet property for identifying the packet. This is the same value you use to retrieve the packet from the packet queue.
		NormalPressure, // The pressure of the pen tip perpendicular to the tablet surface. The greater the pressure on the pen tip, the more ink that is drawn.
		TangentPressure, // The pressure of the pen tip along the plane of the tablet surface.
		ButtonPressure, // The pressure on a pressure sensitive button.
		XTiltOrientation, // The angle between the y,z-plane and the pen and y-axis plane. Applies to a pen cursor. The value is 0 when the pen is perpendicular to the drawing surface and is positive when the pen is to the right of perpendicular.
		YTiltOrientation, // The angle between the x,z-plane and the pen and x-axis plane. Applies to a pen cursor. The value is 0 when the pen is perpendicular to the drawing surface and is positive when the pen is upward or away from the user.
		AzimuthOrientation, // The clockwise rotation of the cursor about the z-axis through a full circular range.
		AltitudeOrientation, // The angle between the axis of the pen and the surface of the tablet. The value is 0 when the pen is parallel to the surface and 90 when the pen is perpendicular to the surface. The values are negative when the pen is inverted.
		TwistOrientation, // The clockwise rotation of the cursor about its own axis.
		PitchRotation, // The packet property that indicates whether the tip is above or below a horizontal line that is perpendicular to the writing surface. Note: This requires a 3D digitizer. The value is positive if the tip is above the line and negative if it is below the line. For example, if you hold the pen in front of you and write on an imaginary wall, the pitch is positive if the tip is above a line extending from you to the wall.
		RollRotation, // The clockwise rotation of the pen around its own axis. Note: This requires a 3D digitizer.
		YawRotation, // The angle of the pen to the left or right around the center of its horizontal axis when the pen is horizontal. Note: This requires a 3D digitizer. If you hold the pen in front of you and write on an imaginary wall, zero yaw indicates that the pen is perpendicular to the wall. The value is negative if the tip is to the left of perpendicular and positive if the tip is to the right of perpendicular.
		Width, // The width of the contact area on a touch digitizer.
		Height, // The height of the contact area on a touch digitizer.
		FingerContactConfidence, // The level of confidence that there was finger contact on a touch digitizer.
		DeviceContactID, // The device contact identifier for a packet.

		Num_Enumerators, // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO QUERY HOW MANY PACKET PROPERTIES THERE ARE.
		Invalid_Enumerator = INDEX_NONE // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO INDICATE THAT A PACKET PROPERTY ENTRY IS INVALID.
	};

	enum class ETabletPropertyMetricUnit : int8
	{
		Default, // Units are unknown.
		Inches,
		Centimeters,
		Degrees,
		Radians,
		Seconds,
		Pounds,
		Grams,

		Num_Enumerators, // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO QUERY HOW MANY PACKET PROPERTIES THERE ARE.
		Invalid_Enumerator = INDEX_NONE // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO INDICATE THAT A PACKET PROPERTY ENTRY IS INVALID.
	};

	using FuncSetProperty = void(FStylusInputPacket&, int32, const int8*);

	struct FPacketProperty
	{
		EPacketPropertyType Type = EPacketPropertyType::Invalid_Enumerator;
		ETabletPropertyMetricUnit MetricUnit = ETabletPropertyMetricUnit::Default;
		int32 Minimum = 0;
		int32 Maximum = 0;
		float Resolution = 0.0f;
		FuncSetProperty* SetProperty = nullptr;

		static constexpr int32 SetPropertyDataBufferLength = 24;
		int8 SetPropertyData[SetPropertyDataBufferLength];
	};

	struct FPacketPropertyConstant
	{
		EPacketPropertyType PacketPropertyType;
		//GUID Guid;
		const wchar_t* StrGuid;
	};

	extern FPacketPropertyConstant PacketPropertyConstants[static_cast<uint32>(EPacketPropertyType::Num_Enumerators)];

	struct FWindowContext
	{
		FVector2d XYScale;
		FVector2d XYMaximum;
		FVector2d WindowSize;
		//FCocoaWindow* WindowHandle;
	};

	class FTabletContext final : public IStylusInputTabletContext
	{
	public:
		explicit FTabletContext(const uint32 ID)
			: ID(ID)
		{}

		virtual uint32 GetID() const override { return ID; }
		virtual FString GetName() const override { return ProductName; }

		virtual uint32 GetProductID() const { return ProductID; }
		virtual FString GetVendorName() const { return VendorName; }
		virtual uint32 GetVendorID() const { return VendorID; }
		virtual FString GetSerialNumber() const { return SerialNumber; }

		virtual FIntRect GetInputRectangle() const override { return InputRectangle; }
		virtual ETabletHardwareCapabilities GetHardwareCapabilities() const override { return HardwareCapabilities; }
		virtual ETabletSupportedProperties GetSupportedProperties() const override { return SupportedProperties; }

		uint32 ID;
		FString ProductName;

		uint32 ProductID;
		FString VendorName;
		uint32 VendorID;
		FString SerialNumber;
		FIntRect InputRectangle;
		ETabletSupportedProperties SupportedProperties = ETabletSupportedProperties::None;
		ETabletHardwareCapabilities HardwareCapabilities = ETabletHardwareCapabilities::None;
		FPacketProperty PacketDescriptions[static_cast<uint32>(EPacketPropertyType::Num_Enumerators)];
	};

	using FTabletContextContainer = Private::TSharedRefDataContainer<FTabletContext, false>;

	class FStylusButton final : IStylusInputStylusButton
	{
	public:
		virtual FString GetID() const override { return ID; }
		virtual FString GetName() const override { return Name; }

		FString ID;
		FString Name;
	};

	class FStylusInfo final : public IStylusInputStylusInfo
	{
	public:
		explicit FStylusInfo(const uint32 ID)
			: ID(ID)
		{}

		virtual uint32 GetID() const override { return ID; }
		virtual FString GetName() const override { return Name; }
		virtual uint32 GetNumButtons() const override { return Buttons.Num(); }

		virtual const IStylusInputStylusButton* GetButton(int32 Index) const override
		{
			return 0 <= Index && Index < Buttons.Num() ? reinterpret_cast<const IStylusInputStylusButton*>(&Buttons[Index]) : nullptr;
		}

		uint32 ID = 0;
		FString Name;
		TArray<FStylusButton> Buttons;
	};

	using FStylusInfoContainer = Private::TSharedRefDataContainer<FStylusInfo, false>;
	bool GetMacTabletDevices(FTabletContextContainer& OutDevices);
}
