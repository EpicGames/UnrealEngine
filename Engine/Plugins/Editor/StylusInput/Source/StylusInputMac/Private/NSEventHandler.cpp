// Copyright Epic Games, Inc. All Rights Reserved.

#include "NSEventHandler.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>
#include <Mac/CocoaWindow.h>

#define LOG_PREAMBLE "NSEventHandler"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Mac
{
	//Info shared by IOHID and NSEventLoop
	static uint32 CurrentTabletContextID = 0;
	static EPenStatus CurrentPenStatus;
	static EPacketType CurrentPacketType;
	
	FNSEventHandler::FNSEventHandler(IStylusInputInstance* Instance,
						   const FCocoaWindow* CocoaWindow,
						   IStylusInputEventHandler* EventHandler)
		: Instance(Instance)
		, CocoaWindow(CocoaWindow)
		, HIDManagerRef(nullptr)
	{
		if (EventHandler)
		{
			// Immediately install an event handler during construction to capture events coming through during initialization.
			AddEventHandler(EventHandler);
		}
	}

	FNSEventHandler::~FNSEventHandler()
	{
		if (HIDManagerRef)
		{
			CFRelease(HIDManagerRef);
			HIDManagerRef = nullptr;
		}
	}

	void FNSEventHandler::StartListen()
	{
		// --- IOHID Setup ---
		// Handling IOHID events to get the device ID of the event sent
		if( IsIOHIDAvailable() )
		{
			LogVerbose(LOG_PREAMBLE, "Start IOHID");
			IOHIDManagerRef hidManager = IOHIDManagerCreate(kCFAllocatorDefault, kIOHIDOptionsTypeNone);
			ensure(hidManager);

			IOHIDManagerSetDeviceMatching(hidManager, NULL);

			// Schedule with runloop
			IOHIDManagerScheduleWithRunLoop(hidManager, CFRunLoopGetCurrent(), kCFRunLoopDefaultMode);

			// Open manager
			IOReturn openResult = IOHIDManagerOpen(hidManager, kIOHIDOptionsTypeNone);
			if (openResult != kIOReturnSuccess) {
				NSLog(@"Failed to open IOHIDManager: 0x%08x", openResult);
			}

			// Register callback
			IOHIDManagerRegisterInputValueCallback(hidManager, HandleHIDEvent, this);

			// Save reference if you want to stop later
			this->HIDManagerRef = hidManager;
		}

		// --- NSEvent Setup ---
		LogVerbose(LOG_PREAMBLE, "Start NSEvent");
		ensure(TabletContextListener == nil);

		NSEventMask mask = NSEventMaskTabletPoint | NSEventMaskTabletProximity | NSEventMaskLeftMouseDragged | NSEventMaskRightMouseDragged | NSEventMaskOtherMouseDragged | NSEventMaskMouseMoved;
		TabletContextListener = ( [NSEvent addLocalMonitorForEventsMatchingMask:mask handler:^(NSEvent* Event) {
			//if( Event.window == Instance->GetWindowContext()->Window )
			return HandleNSEvent(Event); }] );
	}

	void FNSEventHandler::StopListen()
	{
		if( IsIOHIDAvailable() && HIDManagerRef )
		{
			IOHIDManagerUnscheduleFromRunLoop(HIDManagerRef, CFRunLoopGetCurrent(), kCFRunLoopDefaultMode);
			IOHIDManagerClose(HIDManagerRef, kIOHIDOptionsTypeNone);
			CFRelease(HIDManagerRef);
			HIDManagerRef = nullptr;
		}

		if( TabletContextListener == nil )
			return;

		[NSEvent removeMonitor:TabletContextListener];
		TabletContextListener = nil;
	}

	bool FNSEventHandler::AddEventHandler(IStylusInputEventHandler* EventHandler)
	{
		check(EventHandler);

		if (EventHandlers.Contains(EventHandler))
		{
			LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' already exists."), {EventHandler->GetName()}));
			return false;
		}

		EventHandlers.Add(EventHandler);

		LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was added."), {EventHandler->GetName()}));

		return true;
	}


	bool FNSEventHandler::RemoveEventHandler(IStylusInputEventHandler* EventHandler)
	{
		check(EventHandler);

		const bool bWasRemoved = EventHandlers.Remove(EventHandler) > 0;

		if (bWasRemoved)
		{
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was removed."), {EventHandler->GetName()}));
		}

		return bWasRemoved;
	}

	FTabletContextContainer& FNSEventHandler::GetTabletContexts()
	{
		return TabletContexts;
	}

	void FNSEventHandler::DebugEvent(const FString& Message) const
	{
		for (IStylusInputEventHandler* EventHandler : EventHandlers)
		{
			EventHandler->OnDebugEvent(Message, Instance);
		}
	}
	
	bool FNSEventHandler::IsIOHIDAvailable()
	{
		IOHIDManagerRef manager = IOHIDManagerCreate(kCFAllocatorDefault, kIOHIDOptionsTypeNone);
		if (!manager) 
		{
			NSLog(@"IOHIDManager could not be created.");
			return false;
		}

		// Try opening the manager
		IOReturn result = IOHIDManagerOpen(manager, kIOHIDOptionsTypeNone);
		if (result != kIOReturnSuccess) 
		{
			NSLog(@"IOHIDManagerOpen failed: 0x%08x", result);
			CFRelease(manager);
			return false;
		}

		IOHIDManagerSetDeviceMatching(manager, NULL); // NULL matches all devices

		CFSetRef deviceSet = IOHIDManagerCopyDevices(manager);
		bool hasDevices = false;
		if (deviceSet && CFGetTypeID(deviceSet) == CFSetGetTypeID()) 
		{
			CFIndex count = CFSetGetCount(deviceSet);
			hasDevices = (count > 0);
		}

		if (!hasDevices)
		{
			NSLog(@"No HID devices found or permission denied.");
		}
		
		if (deviceSet)
		{
			CFRelease(deviceSet);
			deviceSet = nullptr;
		}

		IOHIDManagerClose(manager, kIOHIDOptionsTypeNone);
		CFRelease(manager);

		return hasDevices;
	}

	void FNSEventHandler::HandleHIDEvent(void* context, IOReturn result, void* sender, IOHIDValueRef value)
	{
		// Retrieve the device
		IOHIDDeviceRef device = IOHIDElementGetDevice(IOHIDValueGetElement(value));

		CFNumberRef productID = (CFNumberRef)IOHIDDeviceGetProperty(device, CFSTR(kIOHIDProductIDKey));

		uint32 ID = 0;
		if (productID)
			CFNumberGetValue(productID, kCFNumberSInt32Type, &ID);

		CurrentTabletContextID = ID;

		IOHIDElementRef elem = IOHIDValueGetElement(value);
		uint32_t elementUsagePage = IOHIDElementGetUsagePage(elem);
		uint32_t elementUsage = IOHIDElementGetUsage(elem);
		CFIndex state = IOHIDValueGetIntegerValue(value);

		if (elementUsagePage == kHIDPage_Digitizer || elementUsagePage >= 0xFF00)
		{
			switch (elementUsage)
			{
			case kHIDUsage_Dig_TipSwitch:
				if (state) CurrentPenStatus = CurrentPenStatus | EPenStatus::CursorIsTouching;
				else CurrentPenStatus = CurrentPenStatus & ~EPenStatus::CursorIsTouching;
				break;
			case kHIDUsage_Dig_BarrelSwitch:
				if (state) CurrentPenStatus = CurrentPenStatus | EPenStatus::BarrelButtonPressed;
				else CurrentPenStatus = CurrentPenStatus & ~EPenStatus::BarrelButtonPressed;
				break;
			case kHIDUsage_Dig_Eraser:
				if (state) CurrentPenStatus = CurrentPenStatus | EPenStatus::CursorIsInverted;
				else CurrentPenStatus = CurrentPenStatus & ~EPenStatus::CursorIsInverted;
				break;
			}
		}

		// Caveat: Internal trackpad doesn't send events via IOHID, meaning we can't grab its ID from this function
	}

	//Set the Altitude and Azimuth fields based on the Tilt field
	void TiltToOrientation(NSPoint Tilt, float& outAzimuth, float& outAltitude)
	{
		outAzimuth = 0;
		if (Tilt.x != 0)
		{
			outAzimuth = PI / 2 - FMath::Atan2(FMath::Cos(Tilt.x) * FMath::Sin(Tilt.y), FMath::Cos(Tilt.y) * FMath::Sin(Tilt.x));
			if (outAzimuth < 0)
				outAzimuth += 2 * PI;
		}

		outAltitude = PI / 2 - FMath::Acos(FMath::Cos(Tilt.x) * FMath::Cos(Tilt.y));

		outAltitude = FMath::RadiansToDegrees(outAltitude);
		outAzimuth = FMath::RadiansToDegrees(outAzimuth);
	}

	NSEvent* FNSEventHandler::HandleNSEvent(NSEvent* Event)
	{
		ensure(Event);

		if (!Event.window)
			return Event;

		PacketStats.NewPacket();
		FStylusInputPacket Packet;

		NSPoint LocalPosition;
		ConvertCocoaEventPositionToSlate(Event, &LocalPosition);

		Packet.X = LocalPosition.x;
		Packet.Y = LocalPosition.y;
		Packet.Z = Event.absoluteZ;

		Packet.TimerTick = (int32)Event.timestamp;
		Packet.SerialNumber = (int32)Event.pointingDeviceSerialNumber;

		Packet.NormalPressure = Event.pressure;
		if (Event.pressure == 0)
		{
			CurrentPenStatus = CurrentPenStatus & ~EPenStatus::CursorIsTouching;
			if (CurrentPacketType == EPacketType::OnDigitizer || CurrentPacketType == EPacketType::StylusDown)
				CurrentPacketType = EPacketType::StylusUp;
			else
				CurrentPacketType = EPacketType::AboveDigitizer;
		}

		Packet.TangentPressure = Event.tangentialPressure;

		float Azimuth = 0.f;
		float Altitude = 0.f;
		TiltToOrientation(Event.tilt, Azimuth, Altitude);

		Packet.AzimuthOrientation = Azimuth;
		Packet.AltitudeOrientation = Altitude;
		Packet.TwistOrientation = Event.rotation;

		Packet.XTiltOrientation = Event.tilt.x;
		Packet.YTiltOrientation = Event.tilt.y;

		Packet.TabletContextID = CurrentTabletContextID;

		if (Event.buttonMask & NSEventButtonMaskPenTip || Event.pressure != 0)
		{
			CurrentPenStatus = CurrentPenStatus | EPenStatus::CursorIsTouching;
			if (CurrentPacketType != EPacketType::OnDigitizer && CurrentPacketType != EPacketType::StylusDown)
				CurrentPacketType = EPacketType::StylusDown;
			else
				CurrentPacketType = EPacketType::OnDigitizer;
		}
		else
		{
			CurrentPenStatus = CurrentPenStatus & ~EPenStatus::CursorIsTouching;
		}

		if (Event.buttonMask & NSEventButtonMaskPenLowerSide) // Inconvenient warning
		{
			CurrentPenStatus = CurrentPenStatus | EPenStatus::BarrelButtonPressed;
		}
		else
		{
			CurrentPenStatus = CurrentPenStatus & ~EPenStatus::BarrelButtonPressed;
		}

		Packet.PenStatus = CurrentPenStatus;
		Packet.Type = CurrentPacketType;

		for (IStylusInputEventHandler* EventHandler : EventHandlers)
		{
			EventHandler->OnPacket(Packet, Instance);
		}
		return Event;
	}

	bool FNSEventHandler::ConvertCocoaEventPositionToSlate(NSEvent* Event, NSPoint* oPoint)
	{
		if (!Event || !oPoint)
			return false;

		if (!CocoaWindow)
			return false;

		*oPoint = Event.locationInWindow;
		// oPoint is relative to the window it is hovering above
		// convert to the window this plugin is used on
		*oPoint = [Event.window convertPointToScreen:*oPoint];
		*oPoint = [CocoaWindow convertPointFromScreen:*oPoint];
		NSSize windowSize = CocoaWindow.contentView.frame.size;

		CGFloat scale = CocoaWindow.backingScaleFactor;

		oPoint->x *= scale;
		oPoint->y = (windowSize.height - oPoint->y) * scale;

		return true;
	}
}

#undef LOG_PREAMBLE
