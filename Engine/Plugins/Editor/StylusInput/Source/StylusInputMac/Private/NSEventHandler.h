// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputPacket.h>
#include <Delegates/Delegate.h>

#include "MacStats.h"
#include "MacTabletContext.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <IOKit/hid/IOHIDManager.h>
#import <Foundation/Foundation.h>

@class FCocoaWindow;

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
	class IStylusInputInstance;
}

namespace UE::StylusInput::Mac
{
	class FNSEventHandler
	{
	public:
		FNSEventHandler(IStylusInputInstance* Instance, const FCocoaWindow* CocoaWindow, IStylusInputEventHandler* EventHandler);
		virtual ~FNSEventHandler();
		
		bool AddEventHandler(IStylusInputEventHandler* EventHandler);
		bool RemoveEventHandler(IStylusInputEventHandler* EventHandler);
		int32 NumEventHandlers() const { return EventHandlers.Num(); }

		float GetPacketsPerSecond() const { return PacketStats.GetPacketsPerSecond(); }

		// Starts listening to events on all tablet contexts we have
		virtual void StartListen();

		// Stops listening to events on all tablet contexts we have
		virtual void StopListen();

		FTabletContextContainer& GetTabletContexts();

		void SetTabletContextID(uint32 ID);
		void SetCurrentPenStatus(EPenStatus);
		void SetCurrentPacketType(EPacketType);

	protected:
		void DebugEvent(const FString& Message) const;

		// Used to get the device ID of the device that sent the last event. We treat the actual event with NSEvent
		static bool IsIOHIDAvailable();
		static void HandleHIDEvent(void* context, IOReturn result, void* sender, IOHIDValueRef value);
		virtual NSEvent* HandleNSEvent(NSEvent* Event);

		bool ConvertCocoaEventPositionToSlate(NSEvent* Event, NSPoint* oPoint);

	protected:
		IStylusInputInstance* const Instance;
		FPacketStats PacketStats;
		const FCocoaWindow* CocoaWindow;
		FTabletContextContainer TabletContexts;
		TArray<IStylusInputEventHandler*> EventHandlers;

		id TabletContextListener = nil;
		IOHIDManagerRef HIDManagerRef;
	};
}
