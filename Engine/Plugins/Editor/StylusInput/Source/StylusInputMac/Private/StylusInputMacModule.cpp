// Copyright Epic Games, Inc. All Rights Reserved.

#include <Modules/ModuleManager.h>

#include "MacInterface.h"

namespace UE::StylusInput::Mac
{
	class FStylusInputMacModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		TUniquePtr<IStylusInputInterface> RegisteredInterface;
	};

	void FStylusInputMacModule::StartupModule()
	{
		RegisteredInterface = FMacInterface::Create();
		if (RegisteredInterface)
		{
			RegisterInterface(RegisteredInterface.Get());
		}
	}

	void FStylusInputMacModule::ShutdownModule()
	{
		if (RegisteredInterface)
		{
			UnregisterInterface(RegisteredInterface.Get());

			RegisteredInterface.Reset();
		}
	}
}

IMPLEMENT_MODULE(UE::StylusInput::Mac::FStylusInputMacModule, StylusInputMac);
