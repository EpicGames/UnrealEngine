// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StylusInputMac : ModuleRules
	{
		public StylusInputMac(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PrivateDependencyModuleNames.AddRange(
				[
					"Core",
					"Slate",
					"SlateCore",
					"StylusInput"
				]
			);

			PublicFrameworks.AddRange(
				[
					"IOKit",
					"CoreFoundation",
					"Foundation"
				]
			);
		}
	}
}