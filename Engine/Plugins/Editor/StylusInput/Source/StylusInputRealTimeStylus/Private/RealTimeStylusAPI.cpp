// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusAPI.h"

#include <KnownFolders.h>
#include <shtypes.h>
#include <StylusInputUtils.h>
#include <Windows/WindowsPlatformProcess.h>

#include "RealTimeStylusUtils.h"

#define LOG_PREAMBLE "RealTimeStylusAPI"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::RealTimeStylus
{
	FRealTimeStylusAPI::FRealTimeStylusAPI()
	{
		bInitializedComLibrary = FWindowsPlatformMisc::CoInitialize();
		if (!bInitializedComLibrary)
		{
			LogError(LOG_PREAMBLE, "Could not initialize COM library.");
			return;
		}

		auto GetDllHandle = [&DllHandles = DllHandles](const TCHAR* DllName) -> void* {
			void* DllHandle = FWindowsPlatformProcess::GetDllHandle(DllName);
			if (DllHandle)
			{
				DllHandles.Add(DllHandle);
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Retrieved DLL handle for {0}."), {DllName}));
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get DLL handle for {0}."), {DllName}));
			}
			return DllHandle;
		};

		auto GetDllExport = []<typename FFuncType>(const TCHAR* DllName, void* DllHandle, const TCHAR* ExportName, FFuncType& ExportHandle)
		{
			void* FuncPtr = FWindowsPlatformProcess::GetDllExport(DllHandle, ExportName);
			if (FuncPtr)
			{
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Retrieved DLL export '{0}' from {1}."), {ExportName, DllName}));
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get DLL export '{0}' in {1}."), {ExportName, DllName}));
			}
			ExportHandle = static_cast<FFuncType>(FuncPtr);
		};

		const TCHAR* Kernel32DllNAme = TEXT("Kernel32.dll");
		if (void* Kernel32DllHandle = GetDllHandle(Kernel32DllNAme))
		{
			GetDllExport(Kernel32DllNAme, Kernel32DllHandle, TEXT("GetTickCount"), GetTickCount);
		}

		const TCHAR* User32DllName = TEXT("User32.dll");
		if (void* User32DllHandle = GetDllHandle(User32DllName))
		{
			GetDllExport(User32DllName, User32DllHandle, TEXT("GetClientRect"), GetClientRect);
			GetDllExport(User32DllName, User32DllHandle, TEXT("GetDC"), GetDC);
			GetDllExport(User32DllName, User32DllHandle, TEXT("ReleaseDC"), ReleaseDC);
		}

		const TCHAR* Gdi32DllName = TEXT("Gdi32.dll");
		if (void* Gdi32DllHandle = GetDllHandle(Gdi32DllName))
		{
			GetDllExport(Gdi32DllName, Gdi32DllHandle, TEXT("GetDeviceCaps"), GetDeviceCaps);
		}

		const TCHAR* Ole32DllName = TEXT("Ole32.dll");
		if (void* Ole32DllHandle = GetDllHandle(Ole32DllName))
		{
			GetDllExport(Ole32DllName, Ole32DllHandle, TEXT("CoCreateInstance"), CoCreateInstance);
			GetDllExport(Ole32DllName, Ole32DllHandle, TEXT("CoCreateFreeThreadedMarshaler"), CoCreateFreeThreadedMarshaler);
			GetDllExport(Ole32DllName, Ole32DllHandle, TEXT("CoTaskMemFree"), CoTaskMemFree);
			GetDllExport(Ole32DllName, Ole32DllHandle, TEXT("StringFromGUID2"), StringFromGUID2);
		}

		const TCHAR* OleAut32DllName = TEXT("OleAut32.dll");
		if (void* OleAut32DllHandle = GetDllHandle(OleAut32DllName))
		{
			GetDllExport(OleAut32DllName, OleAut32DllHandle, TEXT("SysFreeString"), SysFreeString);
			GetDllExport(OleAut32DllName, OleAut32DllHandle, TEXT("VariantClear"), VariantClear);
			GetDllExport(OleAut32DllName, OleAut32DllHandle, TEXT("VariantInit"), VariantInit);
		}

		// Try to get handles to RTSCom.dll and InkObj.dll for WindowsInk.
		{
			const TCHAR* Shell32DllName = TEXT("Shell32.dll");
			if (void* Shell32DllHandle = GetDllHandle(Shell32DllName))
			{
				using FFuncSHGetKnownFolderPath = HRESULT(REFKNOWNFOLDERID, DWORD, HANDLE, PWSTR*);
				FFuncSHGetKnownFolderPath* SHGetKnownFolderPath = nullptr;

				GetDllExport(Shell32DllName, Shell32DllHandle, TEXT("SHGetKnownFolderPath"), SHGetKnownFolderPath);

				if (SHGetKnownFolderPath && CoTaskMemFree)
				{
					PWSTR ProgramFilesCommonPath = nullptr;
					if (Succeeded(SHGetKnownFolderPath(FOLDERID_ProgramFilesCommon, 0, nullptr, &ProgramFilesCommonPath), LOG_PREAMBLE))
					{
						const FString InkDLLDirectory = FString(ProgramFilesCommonPath) + TEXT("\\microsoft shared\\ink");
						FPlatformProcess::PushDllDirectory(*InkDLLDirectory);
						{
							const TCHAR* RTSComDllName = TEXT("RTSCom.dll");
							void* RTSComDllHandle = GetDllHandle(RTSComDllName);
							bHasRTSComDllHandle = RTSComDllHandle != nullptr;

							const TCHAR* InkObjDllName = TEXT("InkObj.dll");
							void* InkObjDllHandle = GetDllHandle(InkObjDllName);
							bHasInkObjDllHandle = InkObjDllHandle != nullptr;
						}
						FPlatformProcess::PopDllDirectory(*InkDLLDirectory);
					}

					CoTaskMemFree(ProgramFilesCommonPath);
				}
			}
		}
	}

	FRealTimeStylusAPI::~FRealTimeStylusAPI()
	{
		for (void* DllHandle : DllHandles)
		{
			FWindowsPlatformProcess::FreeDllHandle(DllHandle);
		}

		if (bInitializedComLibrary)
		{
			FWindowsPlatformMisc::CoUninitialize();
		}
	}

	FName FRealTimeStylusAPI::GetName()
	{
		static FName Name("RealTimeStylus");
		return Name;
	}

	const FRealTimeStylusAPI& FRealTimeStylusAPI::GetInstance()
	{
		static FRealTimeStylusAPI API;
		return API;
	}

	bool FRealTimeStylusAPI::IsValid() const
	{
		bool bSuccess = true;

		bSuccess &= bInitializedComLibrary;
		bSuccess &= GetTickCount != nullptr;
		bSuccess &= GetClientRect != nullptr;
		bSuccess &= GetDC != nullptr;
		bSuccess &= ReleaseDC != nullptr;
		bSuccess &= GetDeviceCaps != nullptr;
		bSuccess &= CoCreateInstance != nullptr;
		bSuccess &= CoCreateFreeThreadedMarshaler != nullptr;
		bSuccess &= CoTaskMemFree != nullptr;
		bSuccess &= StringFromGUID2 != nullptr;
		bSuccess &= SysFreeString != nullptr;
		bSuccess &= VariantClear != nullptr;
		bSuccess &= VariantInit != nullptr;
		bSuccess &= bHasRTSComDllHandle;
		bSuccess &= bHasInkObjDllHandle;

		return bSuccess;
	}
}

#undef LOG_PREAMBLE
