// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Containers/Array.h>

#include <Windows/AllowWindowsPlatformTypes.h>
	#include <windef.h>
	#include <wtypes.h>
	#include <Unknwn.h>
#include <Windows/HideWindowsPlatformTypes.h>

namespace UE::StylusInput::RealTimeStylus
{
	class FRealTimeStylusAPI
	{
	public:
		static FName GetName();
		static const FRealTimeStylusAPI& GetInstance();

		bool IsValid() const;

		using FFuncGetTickCount = DWORD();
		FFuncGetTickCount* GetTickCount = nullptr;

		using FFuncGetClientRect = BOOL(HWND, LPRECT);
		FFuncGetClientRect* GetClientRect = nullptr;

		using FFuncGetDC = HDC(HWND);
		FFuncGetDC* GetDC = nullptr;

		using FFuncReleaseDC = int(HWND, HDC);
		FFuncReleaseDC* ReleaseDC = nullptr;

		using FFuncGetDeviceCaps = int(HDC, int);
		FFuncGetDeviceCaps* GetDeviceCaps = nullptr;

		using FFuncCoCreateInstance = HRESULT(REFCLSID, IUnknown*, DWORD, REFIID, LPVOID*);
		FFuncCoCreateInstance* CoCreateInstance = nullptr;

		using FFuncCoCreateFreeThreadedMarshaler = HRESULT(LPUNKNOWN, LPUNKNOWN*);
		FFuncCoCreateFreeThreadedMarshaler* CoCreateFreeThreadedMarshaler = nullptr;

		using FFuncCoTaskMemFree = void(LPVOID);
		FFuncCoTaskMemFree* CoTaskMemFree = nullptr;

		using FFuncStringFromGUID2 = int(REFGUID, LPOLESTR, int);
		FFuncStringFromGUID2* StringFromGUID2 = nullptr;

		using FFuncSysFreeString = void(BSTR);
		FFuncSysFreeString* SysFreeString = nullptr;

		using FFuncVariantClear = HRESULT(VARIANTARG*);
		FFuncVariantClear* VariantClear = nullptr;

		using FFuncVariantInit = void(VARIANTARG*);
		FFuncVariantInit* VariantInit = nullptr;

	private:

		FRealTimeStylusAPI();
		~FRealTimeStylusAPI();

		TArray<void*> DllHandles;

		bool bInitializedComLibrary = false;
		bool bHasRTSComDllHandle = false;
		bool bHasInkObjDllHandle = false;

	};
}
