// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusInstance.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>
#include <Algo/Transform.h>

#include "RealTimeStylusAPI.h"
#include "RealTimeStylusPluginAsync.h"
#include "RealTimeStylusPluginSync.h"
#include "RealTimeStylusUtils.h"

// Fixed indices at which the plugins are installed; we only ever have one each.
#define ASYNC_PLUGIN_INDEX 0
#define SYNC_PLUGIN_INDEX 0

#define LOCTEXT_NAMESPACE "RealTimeStylusInstance"
#define LOG_PREAMBLE "RealTimeStylusInstance"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::RealTimeStylus
{
	FRealTimeStylusInstance::FRealTimeStylusInstance(const uint32 ID, const HWND OSWindowHandle)
		: ID(ID)
		, RealTimeStylusAPI(FRealTimeStylusAPI::GetInstance())
	{
		SetupWindowContext(OSWindowHandle);

		void* OutInstance = nullptr;
		if (Failed(RealTimeStylusAPI.CoCreateInstance(CLSID_RealTimeStylus, nullptr, CLSCTX_ALL, IID_IRealTimeStylus, &OutInstance), LOG_PREAMBLE))
		{
			LogError(LOG_PREAMBLE, "Could not create RealTimeStylus COM object instance!");
		}
		RealTimeStylus = static_cast<IRealTimeStylus*>(OutInstance);

		if (RealTimeStylus.IsValid())
		{
			TArray<GUID> DesiredPackets;
			DesiredPackets.Reserve(std::size(PacketPropertyConstants));
			Algo::Transform(PacketPropertyConstants, DesiredPackets, [](const FPacketPropertyConstant& Property) { return Property.Guid; });

			if (Failed(RealTimeStylus->SetDesiredPacketDescription(DesiredPackets.Num(), DesiredPackets.GetData()), LOG_PREAMBLE))
			{
				LogError(LOG_PREAMBLE, "Could not set desired RealTimeStylus packet description!");
				bWasInitializedSuccessfully = false;
			}

			if (Failed(RealTimeStylus->put_HWND(reinterpret_cast<HANDLE_PTR>(OSWindowHandle)), LOG_PREAMBLE))
			{
				LogError(LOG_PREAMBLE, "Could not set RealTimeStylus window handle!");
				bWasInitializedSuccessfully = false;
			}

			if (Failed(RealTimeStylus->put_Enabled(::Windows::TRUE), LOG_PREAMBLE))
			{
				LogError(LOG_PREAMBLE, "Could not enable RealTimeStylus interface!");
				bWasInitializedSuccessfully = false;
			}
		}
		else
		{
			LogError(LOG_PREAMBLE, "Stylus input via the RealTimeStylus interface is not available!");
			bWasInitializedSuccessfully = false;
		}

		if (bWasInitializedSuccessfully)
		{
			LogVerbose(LOG_PREAMBLE, "Successfully initialized RealTimeStylus COM object.");
		}
	}

	FRealTimeStylusInstance::~FRealTimeStylusInstance()
	{
		if (RealTimeStylus.IsValid())
		{
			RealTimeStylus->RemoveAllStylusAsyncPlugins();
			RealTimeStylus->RemoveAllStylusSyncPlugins();
		}
	}

	bool FRealTimeStylusInstance::AddEventHandler(IStylusInputEventHandler* EventHandler, const EEventHandlerThread Thread)
	{
		if (!RealTimeStylus.IsValid())
		{
			LogWarning(LOG_PREAMBLE, "Stylus input via the RealTimeStylus interface is not available; event handler was not installed.");
			return false;
		}

		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to add nullptr as event handler.");
			return false;
		}

		if (Thread == EEventHandlerThread::OnGameThread)
		{
			if (!AsyncPlugin)
			{
				EnablePlugin(EEventHandlerThread::OnGameThread, EventHandler);

				if (!AsyncPlugin)
				{
					LogError(LOG_PREAMBLE, FString::Format(
								 TEXT("Event handler '{0}' was not added since the game thread stylus input plugin could not be installed."),
								 {EventHandler->GetName()}));
					return false;
				}

				return true;
			}

			return AsyncPlugin->AddEventHandler(EventHandler);
		}

		if (Thread == EEventHandlerThread::Asynchronous)
		{
			if (!SyncPlugin)
			{
				EnablePlugin(EEventHandlerThread::Asynchronous, EventHandler);

				if (!SyncPlugin)
				{
					LogError(LOG_PREAMBLE, FString::Format(
								 TEXT("Event handler '{0}' was not added since the asynchronous stylus input plugin could not be installed."),
								 {EventHandler->GetName()}));
					return false;
				}

				return true;
			}

			return SyncPlugin->AddEventHandler(EventHandler);
		}

		return false;
	}

	bool FRealTimeStylusInstance::RemoveEventHandler(IStylusInputEventHandler* EventHandler)
	{
		if (!RealTimeStylus.IsValid())
		{
			return false;
		}

		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to remove nullptr event handler.");
			return false;
		}

		bool bWasRemoved = false;

		if (AsyncPlugin && AsyncPlugin->RemoveEventHandler(EventHandler))
		{
			bWasRemoved = true;

			if (AsyncPlugin->NumEventHandlers() == 0)
			{
				DisablePlugin(EEventHandlerThread::OnGameThread);
			}
		}

		if (SyncPlugin && SyncPlugin->RemoveEventHandler(EventHandler))
		{
			bWasRemoved = true;

			if (SyncPlugin->NumEventHandlers() == 0)
			{
				DisablePlugin(EEventHandlerThread::Asynchronous);
			}
		}

		if (!bWasRemoved)
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Event handler {0} does not exist."), {EventHandler->GetName()}));
		}

		return bWasRemoved;
	}

	const TSharedPtr<IStylusInputTabletContext> FRealTimeStylusInstance::GetTabletContext(const uint32 TabletContextID)
	{
		return TabletContexts.Get(TabletContextID);
	}

	const TSharedPtr<IStylusInputStylusInfo> FRealTimeStylusInstance::GetStylusInfo(uint32 StylusID)
	{
		return StylusInfos.Get(StylusID);
	}

	float FRealTimeStylusInstance::GetPacketsPerSecond(const EEventHandlerThread EventHandlerThread) const
	{
		if (EventHandlerThread == EEventHandlerThread::Asynchronous)
		{
			return SyncPlugin ? SyncPlugin->GetPacketsPerSecond() : 0.0f;
		}

		if (EventHandlerThread == EEventHandlerThread::OnGameThread)
		{
			return AsyncPlugin ? AsyncPlugin->GetPacketsPerSecond() : 0.0f;
		}

		return -1.0f;
	}

	FName FRealTimeStylusInstance::GetInterfaceName()
	{
		return FRealTimeStylusAPI::GetName();
	}

	FText FRealTimeStylusInstance::GetName()
	{
		return FText::Format(LOCTEXT("RealTimeStylus", "RealTimeStylus #{0}"), ID);
	}

	bool FRealTimeStylusInstance::WasInitializedSuccessfully() const
	{
		return bWasInitializedSuccessfully;
	}

	void FRealTimeStylusInstance::SetupWindowContext(const HWND HWindow)
	{
		RECT WindowRectangle;
		if (!ensure(RealTimeStylusAPI.GetClientRect(HWindow, &WindowRectangle)))
		{
			LogError(LOG_PREAMBLE, "Could not retrieve window rectangle; failed to setup device context!");
			return;
		}

		if (WindowRectangle.right <= WindowRectangle.left || WindowRectangle.bottom <= WindowRectangle.top)
		{
			LogError(LOG_PREAMBLE, "Window appears to be minimized; failed to setup device context!");
			return;
		}

		const HDC HWindowDeviceContext = RealTimeStylusAPI.GetDC(HWindow);
		if (!ensure(HWindowDeviceContext))
		{
			LogError(LOG_PREAMBLE, "Could not retrieve window device context; failed to setup device context!");
			return;
		}

		auto CleanUp = [this, HWindow, HWindowDeviceContext]
		{
			RealTimeStylusAPI.ReleaseDC(HWindow, HWindowDeviceContext);
		};

		const FVector2d DisplaySizeInMillimeters = {
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, HORZSIZE)),
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, VERTSIZE))
		};

		if (!ensure(DisplaySizeInMillimeters.X > 0.0 && DisplaySizeInMillimeters.Y > 0.0))
		{
			LogError(LOG_PREAMBLE, "Display size in millimeters is invalid; failed to setup device context!");
			CleanUp();
			return;
		}

		const FVector2d DisplaySizeInPixels = {
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, HORZRES)),
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, VERTRES))
		};

		if (!ensure(DisplaySizeInPixels.X > 0.0 && DisplaySizeInPixels.Y > 0.0))
		{
			LogError(LOG_PREAMBLE, "Display size in pixels is invalid; failed to setup device context!");
			CleanUp();
			return;
		}

		const FVector2d DisplayPixelsPerLogicalInch = {
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, LOGPIXELSX)),
			static_cast<double>(RealTimeStylusAPI.GetDeviceCaps(HWindowDeviceContext, LOGPIXELSY))
		};

		if (!ensure(DisplayPixelsPerLogicalInch.X > 0.0 && DisplayPixelsPerLogicalInch.Y > 0.0))
		{
			LogError(LOG_PREAMBLE, "Display pixels per logical inch is invalid; failed to setup device context!");
			CleanUp();
			return;
		}

		const FVector2d WindowSizeInPixels = {
			static_cast<double>(WindowRectangle.right - WindowRectangle.left),
			static_cast<double>(WindowRectangle.bottom - WindowRectangle.top)
		};

		const FVector2d DisplayMillimetersPerPixel = DisplaySizeInMillimeters / DisplaySizeInPixels;
		const FVector2d WindowSizeInMillimeters = WindowSizeInPixels * DisplayMillimetersPerPixel;
		const FVector2d MillimetersPerInch{25.4};
		const FVector2d DisplayInchesPerPixel = MillimetersPerInch / DisplayMillimetersPerPixel;

		WindowContext.XYScale = DisplayInchesPerPixel / DisplayPixelsPerLogicalInch;
		WindowContext.XYMaximum = WindowSizeInMillimeters * FVector2d{100.0} + FVector2d{0.5};
		WindowContext.WindowSize = WindowSizeInPixels;

		CleanUp();

		LogVerbose(LOG_PREAMBLE, "Successfully setup window context.");
	}

	const FWindowContext& FRealTimeStylusInstance::GetWindowContext() const
	{
		return WindowContext;
	}

	void FRealTimeStylusInstance::UpdateStylusInfo(STYLUS_ID StylusID)
	{
		if (!ensure(RealTimeStylus.IsValid()))
		{
			return;
		}

		TSharedPtr<FStylusInfo> StylusInfo = MakeShared<FStylusInfo>(StylusID);

		TComPtr<IInkCursor> InkCursor = nullptr;
		if (Succeeded(RealTimeStylus->GetStylusForId(StylusID, &InkCursor), LOG_PREAMBLE) && InkCursor != nullptr)
		{
			BSTR String = nullptr;
			if (Succeeded(InkCursor->get_Name(&String), LOG_PREAMBLE))
			{
				StylusInfo->Name = String;
				RealTimeStylusAPI.SysFreeString(String);
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get name for stylus with ID {0}."), {static_cast<uint32>(StylusID)}));
			}

			TComPtr<IInkCursorButtons> Buttons = nullptr;
			long NumButtons = 0;
			if (Succeeded(InkCursor->get_Buttons(&Buttons), LOG_PREAMBLE) &&
				Buttons && Succeeded(Buttons->get_Count(&NumButtons), LOG_PREAMBLE))
			{
				if (NumButtons > 0)
				{
					TArray<FStylusButton> StylusInfoButtons;
					StylusInfoButtons.Reserve(NumButtons);

					for (long ButtonIndex = 0; ButtonIndex < NumButtons; ++ButtonIndex)
					{
						VARIANT ButtonIdentifier;
						RealTimeStylusAPI.VariantInit(&ButtonIdentifier);

						ButtonIdentifier.vt = VT_INT;
						ButtonIdentifier.lVal = ButtonIndex;

						TComPtr<IInkCursorButton> Button = nullptr;
						if (Succeeded(Buttons->Item(ButtonIdentifier, &Button), LOG_PREAMBLE))
						{
							FStylusButton NewButton;

							if (Succeeded(Button->get_Id(&String), LOG_PREAMBLE))
							{
								NewButton.ID = String;
								RealTimeStylusAPI.SysFreeString(String);
							}
							else
							{
								LogError(LOG_PREAMBLE, FString::Format(
											 TEXT("Could not get ID for button {0} for stylus with ID {1}."), {static_cast<int32>(ButtonIndex), static_cast<uint32>(StylusID)}));
							}

							if (Succeeded(Button->get_Name(&String), LOG_PREAMBLE))
							{
								NewButton.Name = String;
								RealTimeStylusAPI.SysFreeString(String);
							}
							else
							{
								LogError(LOG_PREAMBLE, FString::Format(
											 TEXT("Could not get name for button {0} for stylus with ID {1}."), {static_cast<int32>(ButtonIndex), static_cast<uint32>(StylusID)}));
							}

							if (!NewButton.ID.IsEmpty() && !NewButton.Name.IsEmpty())
							{
								StylusInfoButtons.Emplace(NewButton);
							}
						}
						else
						{
							LogError(LOG_PREAMBLE, FString::Format(
										 TEXT("Could not get button {0} for stylus with ID {1}."), {static_cast<int32>(ButtonIndex), static_cast<uint32>(StylusID)}));
						}

						RealTimeStylusAPI.VariantClear(&ButtonIdentifier);
					}

					StylusInfo->Buttons = MoveTemp(StylusInfoButtons);
				}
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get buttons for stylus with ID {0}."), {static_cast<uint32>(StylusID)}));
			}
		}
		else
		{
			LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Could not get stylus info for ID {0}."), {static_cast<uint32>(StylusID)}));
		}

		StylusInfos.AddOrReplace(MoveTemp(StylusInfo));
	}

	void FRealTimeStylusInstance::UpdateTabletContexts(const FTabletContextContainer& InTabletContexts)
	{
		TabletContexts.Update(InTabletContexts);
	}

	void FRealTimeStylusInstance::EnablePlugin(const EEventHandlerThread EventHandlerThread, IStylusInputEventHandler* EventHandler)
	{
		if (!ensure(RealTimeStylus.IsValid()))
		{
			return;
		}

		if (EventHandlerThread == EEventHandlerThread::OnGameThread)
		{
			AsyncPlugin = MakeUnique<FRealTimeStylusPluginAsync>(
				this,
				FGetWindowContextCallback::CreateRaw(this, &FRealTimeStylusInstance::GetWindowContext),
				FUpdateTabletContextsCallback::CreateRaw(this, &FRealTimeStylusInstance::UpdateTabletContexts),
				FUpdateStylusInfoCallback::CreateRaw(this, &FRealTimeStylusInstance::UpdateStylusInfo),
				EventHandler
			);

			if (Succeeded(RealTimeStylus->AddStylusAsyncPlugin(ASYNC_PLUGIN_INDEX, AsyncPlugin.Get()), LOG_PREAMBLE))
			{
				LogVerbose(LOG_PREAMBLE, "Added game thread stylus input plugin!");
			}
			else
			{
				LogError(LOG_PREAMBLE, "Could not add game thread stylus input plugin!");
				AsyncPlugin.Reset();
			}
		}

		if (EventHandlerThread == EEventHandlerThread::Asynchronous)
		{
			SyncPlugin = MakeUnique<FRealTimeStylusPluginSync>(
				this,
				FGetWindowContextCallback::CreateRaw(this, &FRealTimeStylusInstance::GetWindowContext),
				FUpdateTabletContextsCallback::CreateRaw(this, &FRealTimeStylusInstance::UpdateTabletContexts),
				FUpdateStylusInfoCallback::CreateRaw(this, &FRealTimeStylusInstance::UpdateStylusInfo),
				EventHandler
			);

			if (Succeeded(SyncPlugin->CreateFreeThreadMarshaler(), LOG_PREAMBLE))
			{
				if (Succeeded(RealTimeStylus->AddStylusSyncPlugin(SYNC_PLUGIN_INDEX, SyncPlugin.Get()), LOG_PREAMBLE))
				{
					LogVerbose(LOG_PREAMBLE, "Added asynchronous stylus input plugin!");
				}
				else
				{
					LogError(LOG_PREAMBLE, "Could not add asynchronous stylus input plugin!");
					SyncPlugin.Reset();
				}
			}
			else
			{
				LogError(LOG_PREAMBLE, "Could not create free thread marshaler for asynchronous stylus input plugin!");
				SyncPlugin.Reset();
			}
		}
	}

	void FRealTimeStylusInstance::DisablePlugin(const EEventHandlerThread EventHandlerThread)
	{
		if (!ensure(RealTimeStylus.IsValid()))
		{
			return;
		}

#pragma warning(push)
#pragma warning(disable : 6387)

		if (EventHandlerThread == EEventHandlerThread::OnGameThread)
		{
			if (Failed(RealTimeStylus->RemoveStylusAsyncPlugin(ASYNC_PLUGIN_INDEX, nullptr), LOG_PREAMBLE))
			{
				LogError(LOG_PREAMBLE, "Could not remove game thread stylus input plugin!");
			}
			else
			{
				LogVerbose(LOG_PREAMBLE, "Removed game thread stylus input plugin!");
			}

			AsyncPlugin.Reset();
		}

		if (EventHandlerThread == EEventHandlerThread::Asynchronous)
		{
			if (Failed(RealTimeStylus->RemoveStylusSyncPlugin(SYNC_PLUGIN_INDEX, nullptr), LOG_PREAMBLE))
			{
				LogError(LOG_PREAMBLE, "Could not remove asynchronous stylus input plugin!");
			}
			else
			{
				LogVerbose(LOG_PREAMBLE, "Removed asynchronous stylus input plugin!");
			}

			SyncPlugin.Reset();
		}

#pragma warning(pop)
	}
}

#undef LOG_PREAMBLE
#undef LOCTEXT_NAMESPACE
