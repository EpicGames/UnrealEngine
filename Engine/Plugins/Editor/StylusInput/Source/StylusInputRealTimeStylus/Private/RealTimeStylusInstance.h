// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <Templates/UniquePtr.h>

#include <Windows/AllowWindowsPlatformTypes.h>
	#include <Microsoft/COMPointer.h>
	#include <RTSCom.h>
#include <Windows/HideWindowsPlatformTypes.h>

#include "RealTimeStylusTabletContext.h"

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
}

namespace UE::StylusInput::RealTimeStylus
{
	class FRealTimeStylusPluginSync;
	class FRealTimeStylusPluginAsync;
	class FRealTimeStylusAPI;

	class FRealTimeStylusInstance : public IStylusInputInstance
	{
	public:
		explicit FRealTimeStylusInstance(uint32 ID, HWND OSWindowHandle);
		virtual ~FRealTimeStylusInstance() override;

		virtual bool AddEventHandler(IStylusInputEventHandler* EventHandler, EEventHandlerThread Thread) override;
		virtual bool RemoveEventHandler(IStylusInputEventHandler* EventHandler) override;

		virtual const TSharedPtr<IStylusInputTabletContext> GetTabletContext(uint32 TabletContextID) override;
		virtual const TSharedPtr<IStylusInputStylusInfo> GetStylusInfo(uint32 StylusID) override;

		virtual float GetPacketsPerSecond(EEventHandlerThread EventHandlerThread) const override;

		virtual FName GetInterfaceName() override;
		virtual FText GetName() override;

		virtual bool WasInitializedSuccessfully() const override;

	private:
		void EnablePlugin(EEventHandlerThread EventHandlerThread, IStylusInputEventHandler* EventHandler);
		void DisablePlugin(EEventHandlerThread EventHandlerThread);

		void SetupWindowContext(HWND HWindow);
		const FWindowContext& GetWindowContext() const;

		void UpdateStylusInfo(STYLUS_ID StylusID);
		void UpdateTabletContexts(const FTabletContextContainer& InTabletContexts);

		const uint32 ID;
		const FRealTimeStylusAPI& RealTimeStylusAPI;
		FWindowContext WindowContext;

		TComPtr<IRealTimeStylus> RealTimeStylus;
		TUniquePtr<FRealTimeStylusPluginAsync> AsyncPlugin;
		TUniquePtr<FRealTimeStylusPluginSync> SyncPlugin;

		FTabletContextThreadSafeContainer TabletContexts;
		FStylusInfoThreadSafeContainer StylusInfos;

		bool bWasInitializedSuccessfully = true;
	};
}
