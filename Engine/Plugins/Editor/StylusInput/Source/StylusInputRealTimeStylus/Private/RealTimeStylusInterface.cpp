// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusInterface.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>

#include <GenericPlatform/GenericWindow.h>
#include <Templates/SharedPointer.h>
#include <Widgets/SWindow.h>

#include <Windows/AllowWindowsPlatformTypes.h>
	#include <RTSCOM_i.c>
#include <Windows/HideWindowsPlatformTypes.h>

#include "RealTimeStylusInstance.h"
#include "RealTimeStylusAPI.h"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::RealTimeStylus
{
	FName FRealTimeStylusInterface::GetName() const
	{
		return FRealTimeStylusAPI::GetName();
	}

	IStylusInputInstance* FRealTimeStylusInterface::CreateInstance(SWindow& Window)
	{
		if (FRefCountedInstance* ExistingRefCountedInstance = Instances.Find(&Window))
		{
			++ExistingRefCountedInstance->RefCount;
			return ExistingRefCountedInstance->Instance.Get();
		}

		HWND OSWindowHandle = [&Window]
		{
			const TSharedPtr<const FGenericWindow> NativeWindow = Window.GetNativeWindow();
			return NativeWindow.IsValid() ? static_cast<HWND>(NativeWindow->GetOSWindowHandle()) : nullptr;
		}();

		if (!OSWindowHandle)
		{
			LogError("RealTimeStylusInterface", "Could not get native window handle.");
			return nullptr;
		}

		FRealTimeStylusInstance* NewInstance = Instances.Emplace(
			&Window, {MakeUnique<FRealTimeStylusInstance>(NextInstanceID++, OSWindowHandle), 1}).Instance.Get();
		if (!ensureMsgf(NewInstance, TEXT("RealTimeStylusInterface: Failed to create stylus input instance.")))
		{
			Instances.Remove(&Window);
			return nullptr;
		}

		return NewInstance;
	}

	bool FRealTimeStylusInterface::ReleaseInstance(IStylusInputInstance* Instance)
	{
		check(Instance);

		FRealTimeStylusInstance *const RealTimeStylusInstance = static_cast<FRealTimeStylusInstance*>(Instance);

		// Find existing instance
		for (TTuple<SWindow*, FRefCountedInstance>& Entry : Instances)
		{
			FRefCountedInstance& RefCountedInstance = Entry.Get<1>();

			if (RefCountedInstance.Instance.Get() == RealTimeStylusInstance)
			{
				// Decrease reference count
				check(RefCountedInstance.RefCount > 0);
				if (--RefCountedInstance.RefCount == 0)
				{
					// Delete if there are no references left
					Instances.Remove(Entry.Key);
				}

				return true;
			}
		}

		ensureMsgf(false, TEXT("RealTimeStylusInterface: Failed to find provided instance."));
		return false;
	}

	TUniquePtr<IStylusInputInterface> FRealTimeStylusInterface::Create()
	{
		if (FRealTimeStylusAPI::GetInstance().IsValid())
		{
			TUniquePtr<FRealTimeStylusInterface> RealTimeStylusInterface = MakeUnique<FRealTimeStylusInterface>();
			return TUniquePtr<IStylusInputInterface>(MoveTemp(RealTimeStylusInterface));
		}

		return nullptr;
	}
}
