// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <StylusInputInterface.h>
#include <Containers/Map.h>
#include <Templates/UniquePtr.h>

namespace UE::StylusInput::RealTimeStylus
{
	class FRealTimeStylusInstance;

	class FRealTimeStylusInterface : public IStylusInputInterface
	{
	public:
		virtual FName GetName() const override;

		virtual IStylusInputInstance* CreateInstance(SWindow& Window) override;
		virtual bool ReleaseInstance(IStylusInputInstance* Instance) override;

	private:
		friend class FStylusInputRealTimeStylusModule;

		static TUniquePtr<IStylusInputInterface> Create();

		struct FRefCountedInstance
		{
			TUniquePtr<FRealTimeStylusInstance> Instance;
			int32 RefCount;
		};

		TMap<SWindow*, FRefCountedInstance> Instances;
		uint32 NextInstanceID = 0;
	};
}
