// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusPluginBase.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>

#include "RealTimeStylusAPI.h"
#include "RealTimeStylusUtils.h"

#include <Windows/AllowWindowsPlatformAtomics.h>
	#include <comdef.h>
#include <Windows/HideWindowsPlatformAtomics.h>

#define LOG_PREAMBLE "RealTimeStylusPluginBase"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::RealTimeStylus
{
	bool SetupTabletContextMetadata(IInkTablet& InkTablet, FTabletContext& TabletContext)
	{
		const FRealTimeStylusAPI& RealTimeStylusAPI = FRealTimeStylusAPI::GetInstance();

		bool bSuccess = true;

		BSTR String = nullptr;
		if (Succeeded(InkTablet.get_Name(&String), LOG_PREAMBLE))
		{
			TabletContext.Name = String;
			RealTimeStylusAPI.SysFreeString(String);
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get name for TabletContext with ID {0}."), {TabletContext.ID}));
			bSuccess = false;
		}

		if (Succeeded(InkTablet.get_PlugAndPlayId(&String), LOG_PREAMBLE))
		{
			TabletContext.PlugAndPlayID = FString(String);
			RealTimeStylusAPI.SysFreeString(String);
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get plug and play ID for TabletContext with ID {0}."), {TabletContext.ID}));
			bSuccess = false;
		}

		TComPtr<IInkRectangle> Rectangle = nullptr;
		if (Succeeded(InkTablet.get_MaximumInputRectangle(&Rectangle), LOG_PREAMBLE))
		{
			long Top, Left, Bottom, Right;
			if (Succeeded(Rectangle->GetRectangle(&Top, &Left, &Bottom, &Right), LOG_PREAMBLE))
			{
				TabletContext.InputRectangle = {Left, Top, Right, Bottom};
			}
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get input rectangle for TabletContext with ID {0}."), {TabletContext.ID}));
			bSuccess = false;
		}

		TabletHardwareCapabilities HardwareCapabilities;
		if (Succeeded(InkTablet.get_HardwareCapabilities(&HardwareCapabilities), LOG_PREAMBLE))
		{
			TabletContext.HardwareCapabilities =
				static_cast<ETabletHardwareCapabilities>(HardwareCapabilities & THWC_Integrated) |
				static_cast<ETabletHardwareCapabilities>(HardwareCapabilities & THWC_CursorMustTouch) |
				static_cast<ETabletHardwareCapabilities>(HardwareCapabilities & THWC_HardProximity) |
				static_cast<ETabletHardwareCapabilities>(HardwareCapabilities & THWC_CursorsHavePhysicalIds);
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get hardware capabilities for TabletContext with ID {0}."), {TabletContext.ID}));
			bSuccess = false;
		}

		return bSuccess;
	}

	// TODO compact number of functions needed to only have differences in behavior, and then use offsetof() to apply to different members
	// TODO XY validate mapping 

	void SetPropertyX(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const double& Scale = reinterpret_cast<const double*>(Data)[0];
		const double& Maximum = reinterpret_cast<const double*>(Data)[1];
		const double& WindowWidth = reinterpret_cast<const double*>(Data)[2];

		Packet.X = Value / Scale / Maximum * WindowWidth;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyX)>);

	void SetPropertyY(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const double& Scale = reinterpret_cast<const double*>(Data)[0];
		const double& Maximum = reinterpret_cast<const double*>(Data)[1];
		const double& WindowHeight = reinterpret_cast<const double*>(Data)[2];

		Packet.Y = Value / Scale / Maximum * WindowHeight;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyY)>);

	void SetPropertyZ(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const int32& Minimum = reinterpret_cast<const int32*>(Data)[0];
		const int32& Maximum = reinterpret_cast<const int32*>(Data)[1];

		Packet.Z = static_cast<float>(Value - Minimum) / (Maximum - Minimum);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyZ)>);

	void SetPropertyPacketStatus(FStylusInputPacket& Packet, const int32 Value, const int8*)
	{
		Packet.PenStatus = static_cast<EPenStatus>(Value);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyPacketStatus)>);

	void SetPropertyTimerTick(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
#if 0
		Packet.TimerTick = Value; // The value provided from the RealTimeStylus API seems to only be non-zero if the stylus is down.
#else
		FRealTimeStylusAPI::FFuncGetTickCount* const FuncGetTickCount =	*reinterpret_cast<decltype(FuncGetTickCount)*>(Data);
		Packet.TimerTick = FuncGetTickCount();
#endif
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyTimerTick)>);

	void SetPropertySerialNumber(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		Packet.SerialNumber = Value;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertySerialNumber)>);

	void SetPropertyNormalPressure(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const int32& Minimum = reinterpret_cast<const int32*>(Data)[0];
		const int32& Maximum = reinterpret_cast<const int32*>(Data)[1];

		Packet.NormalPressure = static_cast<float>(Value - Minimum) / (Maximum - Minimum);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyNormalPressure)>);

	void SetPropertyTangentPressure(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const int32& Minimum = reinterpret_cast<const int32*>(Data)[0];
		const int32& Maximum = reinterpret_cast<const int32*>(Data)[1];

		Packet.TangentPressure = static_cast<float>(Value - Minimum) / (Maximum - Minimum);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyTangentPressure)>);

	void SetPropertyButtonPressure(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const int32& Minimum = reinterpret_cast<const int32*>(Data)[0];
		const int32& Maximum = reinterpret_cast<const int32*>(Data)[1];

		Packet.ButtonPressure = static_cast<float>(Value - Minimum) / (Maximum - Minimum);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyButtonPressure)>);

	void SetPropertyXTiltOrientation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.XTiltOrientation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyXTiltOrientation)>);

	void SetPropertyYTiltOrientation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.YTiltOrientation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyYTiltOrientation)>);

	void SetPropertyAzimuthOrientation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.AzimuthOrientation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyAzimuthOrientation)>);

	void SetPropertyAltitudeOrientation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.AltitudeOrientation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyAltitudeOrientation)>);

	void SetPropertyTwistOrientation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.TwistOrientation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyTwistOrientation)>);

	void SetPropertyPitchRotation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.PitchRotation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyPitchRotation)>);

	void SetPropertyRollRotation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.RollRotation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyRollRotation)>);

	void SetPropertyYawRotation(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.YawRotation = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyYawRotation)>);

	void SetPropertyWidth(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.Width = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyWidth)>);

	void SetPropertyHeight(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const float& Resolution = reinterpret_cast<const float*>(Data)[0];

		Packet.Height = Value / Resolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyHeight)>);

	void SetPropertyFingerContactConfidence(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		const int32& Minimum = reinterpret_cast<const int32*>(Data)[0];
		const int32& Maximum = reinterpret_cast<const int32*>(Data)[1];

		Packet.FingerContactConfidence = static_cast<float>(Value - Minimum) / (Maximum - Minimum);
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyFingerContactConfidence)>);

	void SetPropertyDeviceContactID(FStylusInputPacket& Packet, const int32 Value, const int8* Data)
	{
		Packet.DeviceContactID = Value;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyDeviceContactID)>);

	FPacketPropertyHandler::FFuncSetProperty* GetSetPropertyFunc(const EPacketPropertyType Type)
	{
		switch (Type)
		{
		case EPacketPropertyType::X:
			return &SetPropertyX;
		case EPacketPropertyType::Y:
			return &SetPropertyY;
		case EPacketPropertyType::Z:
			return &SetPropertyZ;
		case EPacketPropertyType::PacketStatus:
			return &SetPropertyPacketStatus;
		case EPacketPropertyType::TimerTick:
			return &SetPropertyTimerTick;
		case EPacketPropertyType::SerialNumber:
			return &SetPropertySerialNumber;
		case EPacketPropertyType::NormalPressure:
			return &SetPropertyNormalPressure;
		case EPacketPropertyType::TangentPressure:
			return &SetPropertyTangentPressure;
		case EPacketPropertyType::ButtonPressure:
			return &SetPropertyButtonPressure;
		case EPacketPropertyType::XTiltOrientation:
			return &SetPropertyXTiltOrientation;
		case EPacketPropertyType::YTiltOrientation:
			return &SetPropertyYTiltOrientation;
		case EPacketPropertyType::AzimuthOrientation:
			return &SetPropertyAzimuthOrientation;
		case EPacketPropertyType::AltitudeOrientation:
			return &SetPropertyAltitudeOrientation;
		case EPacketPropertyType::TwistOrientation:
			return &SetPropertyTwistOrientation;
		case EPacketPropertyType::PitchRotation:
			return &SetPropertyPitchRotation;
		case EPacketPropertyType::RollRotation:
			return &SetPropertyRollRotation;
		case EPacketPropertyType::YawRotation:
			return &SetPropertyYawRotation;
		case EPacketPropertyType::Width:
			return &SetPropertyWidth;
		case EPacketPropertyType::Height:
			return &SetPropertyHeight;
		case EPacketPropertyType::FingerContactConfidence:
			return &SetPropertyFingerContactConfidence;
		case EPacketPropertyType::DeviceContactID:
			return &SetPropertyDeviceContactID;
		default:
			checkSlow(false)
			return nullptr;
		}
	}

	template <int32 DataBufferLength>
	void AssignSetPropertyData(const EPacketPropertyType Type, int8 *const Data, const FWindowContext& WindowContext, const FPacketProperty& Description)
	{
		int8* DataCurrent = Data;

		auto Add = [&DataCurrent]<typename T>(T Value)
		{
			*reinterpret_cast<T*>(DataCurrent) = Value;
			DataCurrent += sizeof(T);
		};

		switch (Type)
		{
		case EPacketPropertyType::X:
			Add(WindowContext.XYScale.X);
			Add(WindowContext.XYMaximum.X);
			Add(WindowContext.WindowSize.X);
			break;
		case EPacketPropertyType::Y:
			Add(WindowContext.XYScale.Y);
			Add(WindowContext.XYMaximum.Y);
			Add(WindowContext.WindowSize.Y);
			break;
		case EPacketPropertyType::Z:
		case EPacketPropertyType::NormalPressure:
		case EPacketPropertyType::TangentPressure:
		case EPacketPropertyType::ButtonPressure:
		case EPacketPropertyType::FingerContactConfidence:
			Add(Description.Minimum);
			Add(Description.Maximum);
			break;
		case EPacketPropertyType::XTiltOrientation:
		case EPacketPropertyType::YTiltOrientation:
		case EPacketPropertyType::AzimuthOrientation:
		case EPacketPropertyType::AltitudeOrientation:
		case EPacketPropertyType::TwistOrientation:
		case EPacketPropertyType::PitchRotation:
		case EPacketPropertyType::RollRotation:
		case EPacketPropertyType::YawRotation:
		case EPacketPropertyType::Width:
		case EPacketPropertyType::Height:
			Add(Description.Resolution);
			break;
		case EPacketPropertyType::TimerTick:
			Add(FRealTimeStylusAPI::GetInstance().GetTickCount);
			break;
		case EPacketPropertyType::PacketStatus:
		case EPacketPropertyType::SerialNumber:
		case EPacketPropertyType::DeviceContactID:
			// do nothing
			break;
		default:
			checkSlow(false)
		}

		check(std::distance(Data, DataCurrent) <= DataBufferLength)
	}

	bool SetupTabletContextPacketDescriptionData(IRealTimeStylus* RealTimeStylus, const FWindowContext& WindowContext, FTabletContext& TabletContext)
	{
		const FRealTimeStylusAPI& RealTimeStylusAPI = FRealTimeStylusAPI::GetInstance();

		auto InvalidatePacketDescriptions = [&PacketDescriptions = TabletContext.PacketProperties](const int32 StartingIndex)
		{
			for (int32 Index = StartingIndex; Index < static_cast<int32>(EPacketPropertyType::Num_Enumerators); ++Index)
			{
				PacketDescriptions[Index] = {};
			}
		};

		FLOAT InkToDeviceScaleX = 0.0f;
		FLOAT InkToDeviceScaleY = 0.0f;
		ULONG PropertiesCount = 0;
		PACKET_PROPERTY* Properties = nullptr;
		if (Succeeded(RealTimeStylus->GetPacketDescriptionData(TabletContext.ID, &InkToDeviceScaleX, &InkToDeviceScaleY, &PropertiesCount, &Properties), LOG_PREAMBLE))
		{
			if (!Properties)
			{
				LogWarning(LOG_PREAMBLE, FString::Format(
					           TEXT("Retrieved nullptr when trying to get packet description data for TabletContext with ID {0}."), {TabletContext.ID}));
				return false;
			}

			if (PropertiesCount == 0)
			{
				LogWarning(LOG_PREAMBLE, FString::Format(
					           TEXT("Retrieved zero packets properties when trying to get packet description data for TabletContext with ID {0}."),
					           {TabletContext.ID}));
				return false;
			}

			for (ULONG Index = 0; Index < PropertiesCount; ++Index)
			{
#pragma warning(push)
#pragma warning(disable : 28199)
				const PACKET_PROPERTY& Property = Properties[Index];
#pragma warning(pop)

				bool bFoundProperty = false;
				for (const FPacketPropertyConstant& PropertyConstant : PacketPropertyConstants)
				{
					if (Property.guid == PropertyConstant.Guid)
					{
						const ETabletSupportedProperties SupportedProperty = static_cast<ETabletSupportedProperties>(
							1 << static_cast<std::underlying_type_t<ETabletSupportedProperties>>(PropertyConstant.PacketPropertyType)
						);
						TabletContext.SupportedProperties = TabletContext.SupportedProperties | SupportedProperty;

						FPacketProperty& PropertyProperty = TabletContext.PacketProperties[Index];
						PropertyProperty.Type = static_cast<EPacketPropertyType>(PropertyConstant.PacketPropertyType);
						PropertyProperty.Minimum = Property.PropertyMetrics.nLogicalMin;
						PropertyProperty.Maximum = Property.PropertyMetrics.nLogicalMax;
						PropertyProperty.MetricUnit = static_cast<ETabletPropertyMetricUnit>(Property.PropertyMetrics.Units);
						PropertyProperty.Resolution = Property.PropertyMetrics.fResolution;

						FPacketPropertyHandler& PropertyPropertyHandler = TabletContext.PacketPropertyHandlers[Index];
						PropertyPropertyHandler.SetProperty = GetSetPropertyFunc(PropertyConstant.PacketPropertyType);
						AssignSetPropertyData<FPacketPropertyHandler::SetPropertyDataBufferLength>(
							PropertyConstant.PacketPropertyType, PropertyPropertyHandler.SetPropertyData, WindowContext, PropertyProperty);

						if (static_cast<int8>(PropertyProperty.MetricUnit) >= static_cast<int8>(ETabletPropertyMetricUnit::Num_Enumerators))
						{
							PropertyProperty.MetricUnit = ETabletPropertyMetricUnit::Default;

							LogWarning(LOG_PREAMBLE, FString::Format(
									   TEXT("Encountered unknown metric unit value '{0}' while evaluating packet description data for TabletContext with ID {1}."),
									   {static_cast<int8>(PropertyProperty.MetricUnit), TabletContext.ID}));
						}

						bFoundProperty = true;
						break;
					}
				}

				if (!bFoundProperty)
				{
					wchar_t GUIDStringBuffer[64];
					const int32 GUIDStringLength = RealTimeStylusAPI.StringFromGUID2(Property.guid, GUIDStringBuffer, std::size(GUIDStringBuffer));
					if (ensure(GUIDStringLength > 0))
					{
						LogWarning(LOG_PREAMBLE, FString::Format(
							           TEXT("Encountered unknown property '{0}' while evaluating packet description data for TabletContext with ID {1}."),
							           {GUIDStringBuffer, TabletContext.ID}));
					}
				}
			}

			RealTimeStylusAPI.CoTaskMemFree(Properties);

			InvalidatePacketDescriptions(PropertiesCount);
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get packet description data for TabletContext with ID {0}."), {TabletContext.ID}));

			InvalidatePacketDescriptions(0);

			return false;
		}

		return true;
	}

	bool SetupTabletContext(IRealTimeStylus* RealTimeStylus, const FWindowContext& WindowContext, FTabletContext& TabletContext)
	{
		TComPtr<IInkTablet> InkTablet = nullptr;

		if (Failed(RealTimeStylus->GetTabletFromTabletContextId(TabletContext.ID, &InkTablet), LOG_PREAMBLE))
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get tablet context data for ID {0}."), {TabletContext.ID}));
			return false;
		}

		bool bSuccess = true;

		bSuccess &= SetupTabletContextMetadata(*InkTablet, TabletContext);
		bSuccess &= SetupTabletContextPacketDescriptionData(RealTimeStylus, WindowContext, TabletContext);

		return bSuccess;
	}

	bool FRealTimeStylusPluginBase::AddEventHandler(IStylusInputEventHandler* EventHandler)
	{
		check(EventHandler)

		if (EventHandlers.Contains(EventHandler))
		{
			LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' already exists in {1} plugin."), {EventHandler->GetName(), GetName()}));
			return false;
		}

		EventHandlers.Add(EventHandler);

		LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was added to {1} plugin."), {EventHandler->GetName(), GetName()}));

		return true;
	}

	bool FRealTimeStylusPluginBase::RemoveEventHandler(IStylusInputEventHandler* EventHandler)
	{
		check(EventHandler)

		const bool bWasRemoved = EventHandlers.Remove(EventHandler) > 0;

		if (bWasRemoved)
		{
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was removed from {1} plugin."), {EventHandler->GetName(), GetName()}));
		}

		return bWasRemoved;
	}

	FRealTimeStylusPluginBase::FRealTimeStylusPluginBase(IStylusInputInstance* Instance,
	                                                     FGetWindowContextCallback&& GetWindowContextCallback,
	                                                     FUpdateTabletContextsCallback&& UpdateTabletContextsCallback,
	                                                     FUpdateStylusInfoCallback&& UpdateStylusInfoCallback)
		: Instance(Instance)
		, GetWindowContextCallback(MoveTemp(GetWindowContextCallback))
		, UpdateTabletContextsCallback(MoveTemp(UpdateTabletContextsCallback))
		, UpdateStylusInfoCallback(MoveTemp(UpdateStylusInfoCallback))
	{
		check(this->GetWindowContextCallback.IsBound())
		check(this->UpdateTabletContextsCallback.IsBound())
	}

	void FRealTimeStylusPluginBase::DebugEvent(const FString& Message) const
	{
		for (IStylusInputEventHandler* EventHandler : EventHandlers)
		{
			EventHandler->OnDebugEvent(Message, Instance);
		}
	}

	HRESULT FRealTimeStylusPluginBase::ProcessDataInterest(RealTimeStylusDataInterest* DataInterest)
	{
		constexpr bool bGetAllData = false;

		if (bGetAllData)
		{
			*DataInterest = RTSDI_AllData;

			DebugEvent("Requested all data from stylus input.");
		}
		else
		{
			*DataInterest = static_cast<RealTimeStylusDataInterest>(
				RTSDI_Error |
				RTSDI_RealTimeStylusEnabled | RTSDI_RealTimeStylusDisabled |
				RTSDI_InAirPackets | RTSDI_Packets |
				RTSDI_StylusInRange | RTSDI_StylusDown | RTSDI_StylusUp);

			DebugEvent("Requested data for Error, RealTimeStylusEnabled, RealTimeStylusDisabled, InAirPackets, Packets, StylusInRange, StylusDown, "
				"StylusUp from stylus input (DataInterest event).");
		}

		return S_OK;
	}

	HRESULT FRealTimeStylusPluginBase::ProcessError(RealTimeStylusDataInterest DataInterest, const HRESULT ErrorCode)
	{
		constexpr bool bShowAllErrors = false;

		if (bShowAllErrors || ErrorCode != E_NOTIMPL)
		{
			const char* DataInterestStr = [DataInterest]
			{
				switch (DataInterest)
				{
				case RTSDI_AllData:
					return "AllData";
				case RTSDI_None:
					return "None";
				case RTSDI_Error:
					return "Error";
				case RTSDI_RealTimeStylusEnabled:
					return "RealTimeStylusEnabled";
				case RTSDI_RealTimeStylusDisabled:
					return "RealTimeStylusDisabled";
				case RTSDI_StylusNew:
					return "StylusNew";
				case RTSDI_StylusInRange:
					return "StylusInRange";
				case RTSDI_InAirPackets:
					return "InAirPackets";
				case RTSDI_StylusOutOfRange:
					return "StylusOutOfRange";
				case RTSDI_StylusDown:
					return "StylusDown";
				case RTSDI_Packets:
					return "Packets";
				case RTSDI_StylusUp:
					return "StylusUp";
				case RTSDI_StylusButtonUp:
					return "StylusButtonUp";
				case RTSDI_StylusButtonDown:
					return "StylusButtonDown";
				case RTSDI_SystemEvents:
					return "SystemEvents";
				case RTSDI_TabletAdded:
					return "TabletAdded";
				case RTSDI_TabletRemoved:
					return "TabletRemoved";
				case RTSDI_CustomStylusDataAdded:
					return "CustomStylusDataAdded";
				case RTSDI_UpdateMapping:
					return "UpdateMapping";
				case RTSDI_DefaultEvents:
					return "DefaultEvents";
				}
				return "<unknown>";
			}();

			const FString ErrorMessage = FString::Format(TEXT("Error in {0} plugin: {1}, Error={2} ({3})."), {
				                                             GetName(), DataInterestStr, _com_error(ErrorCode).ErrorMessage(), static_cast<int32>(ErrorCode)
			                                             });

			DebugEvent(ErrorMessage);
		}

		return S_OK;
	}

	HRESULT FRealTimeStylusPluginBase::ProcessPackets(const StylusInfo* StylusInfo, uint32 PacketCount, uint32 PacketBufferLength, EPacketType Type,
	                                                  const int32* PacketBuffer)
	{
		checkSlow(StylusInfo)
		checkSlow(PacketBuffer)

		const FPacketPropertyHandler* Handlers = GetPacketPropertyHandlers(StylusInfo->tcid);
		if (!Handlers)
		{
			return E_FAIL;
		}

		const uint32 PropertyCount = PacketBufferLength / PacketCount;
		checkSlow(PropertyCount < static_cast<uint32>(EPacketPropertyType::Num_Enumerators));

		for (uint32 PacketIndex = 0; PacketIndex < PacketCount; PacketIndex += PropertyCount)
		{
			PacketStats.NewPacket();

			FStylusInputPacket Packet{StylusInfo->tcid, StylusInfo->cid, Type};

			for (uint32 PropertyIndex = 0; PropertyIndex < PropertyCount; ++PropertyIndex)
			{
				const FPacketPropertyHandler& Handler = Handlers[PropertyIndex];
				checkSlow(Handler.SetProperty)
				Handler.SetProperty(Packet, PacketBuffer[PropertyIndex], Handler.SetPropertyData);
			}

			for (IStylusInputEventHandler* EventHandler : EventHandlers)
			{
				EventHandler->OnPacket(Packet, Instance);
			}

			PacketBuffer += PropertyCount;
		}

		return S_OK;
	}

	HRESULT FRealTimeStylusPluginBase::ProcessRealTimeStylusEnabled(IRealTimeStylus* RealTimeStylus, const uint32 TabletContextIDsCount,
																   const TABLET_CONTEXT_ID* TabletContextIDs)
	{
		DebugEvent("Stylus input was enabled (RealTimeStylusEnabled event).");

		return UpdateTabletContexts(RealTimeStylus, TabletContextIDsCount, TabletContextIDs);
	}

	HRESULT FRealTimeStylusPluginBase::ProcessRealTimeStylusDisabled(IRealTimeStylus* RealTimeStylus, const uint32 TabletContextIDsCount,
																	const TABLET_CONTEXT_ID* TabletContextIDs)
	{
		DebugEvent("Stylus input was disabled (RealTimeStylusDisabled event).");

		return UpdateTabletContexts(RealTimeStylus, TabletContextIDsCount, TabletContextIDs);
	}

	HRESULT FRealTimeStylusPluginBase::ProcessStylusInRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID)
	{
		UpdateStylusInfoCallback.Execute(StylusID);
		return S_OK;
	}

	HRESULT FRealTimeStylusPluginBase::ProcessTabletAdded(IInkTablet* Tablet)
	{
		// After a TabletAdded event, a RealTimeStylusDisabled event will fire directly followed by a RealTimeStylusEnabled event.
		// We are using these two events instead to update the tablet contexts.

		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginBase::ProcessTabletRemoved(LONG TabletIndex)
	{
		// For simplicity, we don't remove a tablet context when this event is received.
		// However, since there are no more packets coming through for the removed tablet there should be nothing that's continuing to access the outdated
		// tablet context data.

		return E_NOTIMPL;
	}

	const FPacketPropertyHandler* FRealTimeStylusPluginBase::GetPacketPropertyHandlers(uint32 TabletContextID) const
	{
		const TSharedPtr<FTabletContext> TabletContext = TabletContexts.Get(TabletContextID);
		return TabletContext ? TabletContext->PacketPropertyHandlers : nullptr;
	}

	HRESULT FRealTimeStylusPluginBase::UpdateTabletContexts(IRealTimeStylus* RealTimeStylus, uint32 TabletContextIDsNum, const TABLET_CONTEXT_ID* TabletContextIDs)
	{
		const FWindowContext& WindowContext = GetWindowContextCallback.Execute();

		bool bSuccess = true;

		// Remove outdated tablet contexts
		uint32 TabletContextIndex = 0;
		while (TabletContextIndex < TabletContexts.Num())
		{
			const uint32 ExistingTabletContextID = TabletContexts[TabletContextIndex]->ID;

			const bool bContainsID = [ExistingTabletContextID, TabletContextIDsNum, TabletContextIDs]
			{
				for (uint32 TabletContextIDsIndex = 0; TabletContextIDsIndex < TabletContextIDsNum; ++TabletContextIDsIndex)
				{
					if (ExistingTabletContextID == TabletContextIDs[TabletContextIDsIndex])
					{
						return true;
					}
				}
				return false;
			}();

			if (bContainsID)
			{
				++TabletContextIndex;
			}
			else
			{
				TabletContexts.Remove(ExistingTabletContextID);

				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Removed tablet context data for ID {0}."), {ExistingTabletContextID}));
			}
		}

		// Add new tablet contexts
		for (uint32 TabletContextIDsIndex = 0; TabletContextIDsIndex < TabletContextIDsNum; ++TabletContextIDsIndex)
		{
			const uint32 TabletContextID = TabletContextIDs[TabletContextIDsIndex];
			if (!TabletContexts.Contains(TabletContextID))
			{
				const TSharedRef<FTabletContext> TabletContextRef = TabletContexts.Add(TabletContextID);
				FTabletContext& TabletContext = TabletContextRef.Get();

				bSuccess &= SetupTabletContext(RealTimeStylus, WindowContext, TabletContext);

				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Added TabletContext for ID {0} [{1}, {2}]."), {
															TabletContext.ID, TabletContext.Name, TabletContext.PlugAndPlayID
														}));
			}
		}

		UpdateTabletContextsCallback.Execute(TabletContexts);

		return bSuccess ? S_OK : E_FAIL;
	}
}

#undef LOG_PREAMBLE
