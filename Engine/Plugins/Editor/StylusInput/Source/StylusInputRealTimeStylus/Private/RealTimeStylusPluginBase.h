// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputPacket.h>
#include <Delegates/Delegate.h>

#include "RealTimeStylusStats.h"
#include "RealTimeStylusTabletContext.h"

#include <Windows/AllowWindowsPlatformTypes.h>
	#include <Microsoft/COMPointer.h>
	#include <RTSCom.h>
#include <Windows/HideWindowsPlatformTypes.h>

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
	class IStylusInputInstance;
}

namespace UE::StylusInput::RealTimeStylus
{
	class FRealTimeStylusAPI;

	DECLARE_DELEGATE_RetVal(const FWindowContext&, FGetWindowContextCallback);
	DECLARE_DELEGATE_OneParam(FUpdateTabletContextsCallback, const FTabletContextContainer&);
	DECLARE_DELEGATE_OneParam(FUpdateStylusInfoCallback, STYLUS_ID);

	class FRealTimeStylusPluginBase
	{
	public:
		bool AddEventHandler(IStylusInputEventHandler* EventHandler);
		bool RemoveEventHandler(IStylusInputEventHandler* EventHandler);
		int32 NumEventHandlers() const { return EventHandlers.Num(); }

		float GetPacketsPerSecond() const { return PacketStats.GetPacketsPerSecond(); }

	protected:
		FRealTimeStylusPluginBase(IStylusInputInstance* Instance, FGetWindowContextCallback&& GetWindowContextCallback,
		                          FUpdateTabletContextsCallback&& UpdateTabletContextsCallback, FUpdateStylusInfoCallback&& UpdateStylusInfoCallback);
		virtual ~FRealTimeStylusPluginBase() = default;

		virtual FString GetName() const = 0;

		void DebugEvent(const FString& Message) const;

		HRESULT ProcessDataInterest(RealTimeStylusDataInterest* DataInterest);
		HRESULT ProcessError(RealTimeStylusDataInterest DataInterest, HRESULT ErrorCode);
		HRESULT ProcessPackets(const StylusInfo* StylusInfo, uint32 PacketCount, uint32 PacketBufferLength, EPacketType Type, const int32* PacketBuffer);
		HRESULT ProcessRealTimeStylusEnabled(IRealTimeStylus* RealTimeStylus, uint32 TabletContextIDsCount, const TABLET_CONTEXT_ID* TabletContextIDs);
		HRESULT ProcessRealTimeStylusDisabled(IRealTimeStylus* RealTimeStylus, uint32 TabletContextIDsCount, const TABLET_CONTEXT_ID* TabletContextIDs);
		HRESULT ProcessStylusInRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID);
		HRESULT ProcessTabletAdded(IInkTablet* Tablet);
		HRESULT ProcessTabletRemoved(LONG TabletIndex);

	private:
		const FPacketPropertyHandler* GetPacketPropertyHandlers(uint32 TabletContextID) const;
		HRESULT UpdateTabletContexts(IRealTimeStylus* RealTimeStylus, uint32 TabletContextIDsNum, const TABLET_CONTEXT_ID* TabletContextIDs);

		IStylusInputInstance *const Instance;
		FPacketStats PacketStats;
		FGetWindowContextCallback GetWindowContextCallback;
		FUpdateTabletContextsCallback UpdateTabletContextsCallback;
		FUpdateStylusInfoCallback UpdateStylusInfoCallback;
		FTabletContextContainer TabletContexts;
		TArray<IStylusInputEventHandler*> EventHandlers;
	};
}
