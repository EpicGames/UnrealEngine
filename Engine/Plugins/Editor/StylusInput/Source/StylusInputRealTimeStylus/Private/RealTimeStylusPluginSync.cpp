// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusPluginSync.h"

#include "RealTimeStylusAPI.h"

namespace UE::StylusInput::RealTimeStylus
{
	FRealTimeStylusPluginSync::FRealTimeStylusPluginSync(IStylusInputInstance* Instance,
	                                                     FGetWindowContextCallback&& GetWindowContext,
	                                                     FUpdateTabletContextsCallback&& UpdateTabletContextsCallback,
	                                                     FUpdateStylusInfoCallback UpdateStylusInfoCallback,
	                                                     IStylusInputEventHandler* EventHandler)
		: FRealTimeStylusPluginBase(Instance, MoveTemp(GetWindowContext), MoveTemp(UpdateTabletContextsCallback), MoveTemp(UpdateStylusInfoCallback))
	{
		if (EventHandler)
		{
			// Immediately install an event handler during construction to capture events coming through during plugin initialization.
			AddEventHandler(EventHandler);
		}
	}

	FRealTimeStylusPluginSync::~FRealTimeStylusPluginSync()
	{
		if (FreeThreadedMarshaler)
		{
			FreeThreadedMarshaler->Release();
		}
	}

	HRESULT FRealTimeStylusPluginSync::CreateFreeThreadMarshaler()
	{
		check(FreeThreadedMarshaler == nullptr);

		const FRealTimeStylusAPI& RealTimeStylusAPI = FRealTimeStylusAPI::GetInstance();
		return RealTimeStylusAPI.CoCreateFreeThreadedMarshaler(this, &FreeThreadedMarshaler);
	}

	HRESULT FRealTimeStylusPluginSync::RealTimeStylusEnabled(IRealTimeStylus* RealTimeStylus, const ULONG TabletContextIDsCount,
	                                                             const TABLET_CONTEXT_ID* TabletContextIDs)
	{
		return ProcessRealTimeStylusEnabled(RealTimeStylus, TabletContextIDsCount, TabletContextIDs);
	}

	HRESULT FRealTimeStylusPluginSync::RealTimeStylusDisabled(IRealTimeStylus* RealTimeStylus, const ULONG TabletContextIDsCount,
	                                                              const TABLET_CONTEXT_ID* TabletContextIDs)
	{
		return ProcessRealTimeStylusDisabled(RealTimeStylus, TabletContextIDsCount, TabletContextIDs);
	}

	HRESULT FRealTimeStylusPluginSync::StylusDown(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, const ULONG PropertyCount,
	                                                  LONG* PacketBuffer, LONG**)
	{
		return ProcessPackets(StylusInfo, 1, PropertyCount, EPacketType::StylusDown, reinterpret_cast<int32*>(PacketBuffer));
	}

	HRESULT FRealTimeStylusPluginSync::StylusUp(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, const ULONG PropertyCount,
	                                                LONG* PacketBuffer, LONG**)
	{
		return ProcessPackets(StylusInfo, 1, PropertyCount, EPacketType::StylusUp, reinterpret_cast<int32*>(PacketBuffer));
	}

	HRESULT FRealTimeStylusPluginSync::TabletAdded(IRealTimeStylus* RealTimeStylus, IInkTablet* Tablet)
	{
		return ProcessTabletAdded(Tablet);
	}

	HRESULT FRealTimeStylusPluginSync::TabletRemoved(IRealTimeStylus* RealTimeStylus, const LONG TabletIndex)
	{
		return ProcessTabletRemoved(TabletIndex);
	}

	HRESULT FRealTimeStylusPluginSync::InAirPackets(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, const ULONG PacketCount,
	                                                    const ULONG PacketBufferLength, LONG* PacketBuffer, ULONG*, LONG**)
	{
		return ProcessPackets(StylusInfo, PacketCount, PacketBufferLength, EPacketType::AboveDigitizer, reinterpret_cast<int32*>(PacketBuffer));
	}

	HRESULT FRealTimeStylusPluginSync::Packets(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, const ULONG PacketCount,
	                                               const ULONG PacketBufferLength, LONG* PacketBuffer, ULONG*, LONG**)
	{
		return ProcessPackets(StylusInfo, PacketCount, PacketBufferLength, EPacketType::OnDigitizer, reinterpret_cast<int32*>(PacketBuffer));
	}

	HRESULT FRealTimeStylusPluginSync::StylusInRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID)
	{
		return ProcessStylusInRange(RealTimeStylus, TabletContextID, StylusID);
	}

	HRESULT FRealTimeStylusPluginSync::StylusOutOfRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::StylusButtonDown(IRealTimeStylus* RealTimeStylus, STYLUS_ID StylusID, const GUID* GuidStylusButton, POINT* StylusPos)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::StylusButtonUp(IRealTimeStylus* RealTimeStylus, STYLUS_ID StylusID, const GUID* GuidStylusButton, POINT* StylusPos)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::CustomStylusDataAdded(IRealTimeStylus* RealTimeStylus, const GUID* GuidId, ULONG DataCount, const BYTE* Data)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::SystemEvent(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID,
	                                                   SYSTEM_EVENT Event, SYSTEM_EVENT_DATA EventData)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::Error(IRealTimeStylus*, IStylusPlugin* Plugin, const RealTimeStylusDataInterest DataInterest,
	                                             const HRESULT ErrorCode, LONG_PTR*)
	{
		return ProcessError(DataInterest, ErrorCode);
	}

	HRESULT FRealTimeStylusPluginSync::UpdateMapping(IRealTimeStylus* RealTimeStylus)
	{
		return E_NOTIMPL;
	}

	HRESULT FRealTimeStylusPluginSync::DataInterest(RealTimeStylusDataInterest* DataInterest)
	{
		return ProcessDataInterest(DataInterest);
	}
}
