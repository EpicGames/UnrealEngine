// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RealTimeStylusPluginBase.h"

namespace UE::StylusInput::RealTimeStylus
{
	class FRealTimeStylusPluginSync final : public IStylusSyncPlugin, public FRealTimeStylusPluginBase
	{
	public:
		FRealTimeStylusPluginSync(IStylusInputInstance* Instance, FGetWindowContextCallback&& GetWindowContext,
		                          FUpdateTabletContextsCallback&& UpdateTabletContextsCallback, FUpdateStylusInfoCallback UpdateStylusInfoCallback,
		                          IStylusInputEventHandler* EventHandler);
		virtual ~FRealTimeStylusPluginSync() override;

		HRESULT CreateFreeThreadMarshaler();

		// IUnknown
		virtual HRESULT QueryInterface(REFIID InterfaceID, void** InterfaceObject) override
		{
			if (InterfaceID == IID_IStylusSyncPlugin || InterfaceID == IID_IUnknown)
			{
				*InterfaceObject = this;
				AddRef();
				return S_OK;
			}

			if (InterfaceID == IID_IMarshal && FreeThreadedMarshaler)
			{
				return FreeThreadedMarshaler->QueryInterface(InterfaceID, InterfaceObject);
			}

			*InterfaceObject = nullptr;
			return E_NOINTERFACE;
		}

		virtual ULONG AddRef() override
		{
			return ++RefCount;
		}

		virtual ULONG Release() override
		{
			const int32 NewRefCount = --RefCount;
			if (NewRefCount == 0)
				delete this;

			return NewRefCount;
		}

		// IStylusPlugin
		virtual HRESULT RealTimeStylusEnabled(IRealTimeStylus* RealTimeStylus, ULONG TabletContextIDsCount, const TABLET_CONTEXT_ID* TabletContextIDs) override;
		virtual HRESULT RealTimeStylusDisabled(IRealTimeStylus* RealTimeStylus, ULONG TabletContextIDsCount, const TABLET_CONTEXT_ID* TabletContextIDs) override;
		virtual HRESULT StylusDown(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, ULONG PropertyCount, LONG* PacketBuffer, LONG** InOutPkt) override;
		virtual HRESULT StylusUp(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, ULONG PropertyCount, LONG* PacketBuffer, LONG** InOutPkt) override;
		virtual HRESULT TabletAdded(IRealTimeStylus* RealTimeStylus, IInkTablet* Tablet) override;
		virtual HRESULT TabletRemoved(IRealTimeStylus* RealTimeStylus, LONG TabletIndex) override;
		virtual HRESULT InAirPackets(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, ULONG PacketCount, ULONG PacketBufferLength, LONG* PacketBuffer, ULONG* InOutPacketCount, LONG** InOutPacketBuffer) override;
		virtual HRESULT Packets(IRealTimeStylus* RealTimeStylus, const StylusInfo* StylusInfo, ULONG PacketCount, ULONG PacketBufferLength, LONG* PacketBuffer, ULONG* InOutPacketCount, LONG** InOutPacketBuffer) override;
		virtual HRESULT StylusInRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID) override;
		virtual HRESULT StylusOutOfRange(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID) override;
		virtual HRESULT StylusButtonDown(IRealTimeStylus* RealTimeStylus, STYLUS_ID StylusID, const GUID* GuidStylusButton, POINT* StylusPos) override;
		virtual HRESULT StylusButtonUp(IRealTimeStylus* RealTimeStylus, STYLUS_ID StylusID, const GUID* GuidStylusButton, POINT* StylusPos) override;
		virtual HRESULT CustomStylusDataAdded(IRealTimeStylus* RealTimeStylus, const GUID* GuidId, ULONG DataCount, const BYTE* Data) override;
		virtual HRESULT SystemEvent(IRealTimeStylus* RealTimeStylus, TABLET_CONTEXT_ID TabletContextID, STYLUS_ID StylusID, SYSTEM_EVENT Event, SYSTEM_EVENT_DATA EventData) override;
		virtual HRESULT Error(IRealTimeStylus* RealTimeStylus, IStylusPlugin* Plugin, RealTimeStylusDataInterest DataInterest, HRESULT ErrorCode, LONG_PTR* InternalKey) override;
		virtual HRESULT UpdateMapping(IRealTimeStylus* RealTimeStylus) override;
		virtual HRESULT DataInterest(RealTimeStylusDataInterest* DataInterest) override;

	protected:
		// FRealTimeStylusPluginBase
		virtual FString GetName() const override { return "Asynchronous"; }

	private:
		int32 RefCount = 1;

		IUnknown* FreeThreadedMarshaler = nullptr;
	};
}
