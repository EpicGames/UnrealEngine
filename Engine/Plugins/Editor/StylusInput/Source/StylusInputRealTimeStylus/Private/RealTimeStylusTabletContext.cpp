// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusTabletContext.h"

#include <Microsoft/COMPointer.h>
#include <msinkaut.h>

namespace UE::StylusInput::RealTimeStylus
{
	FPacketPropertyConstant PacketPropertyConstants[] = {
		{EPacketPropertyType::X,                       GUID_PACKETPROPERTY_GUID_X,                        STR_GUID_X                      },
		{EPacketPropertyType::Y,                       GUID_PACKETPROPERTY_GUID_Y,                        STR_GUID_Y                      },
		{EPacketPropertyType::Z,                       GUID_PACKETPROPERTY_GUID_Z,                        STR_GUID_Z                      },
		{EPacketPropertyType::PacketStatus,            GUID_PACKETPROPERTY_GUID_PACKET_STATUS,            STR_GUID_PAKETSTATUS            },
		{EPacketPropertyType::TimerTick,               GUID_PACKETPROPERTY_GUID_TIMER_TICK,               STR_GUID_TIMERTICK              },
		{EPacketPropertyType::SerialNumber,            GUID_PACKETPROPERTY_GUID_SERIAL_NUMBER,            STR_GUID_SERIALNUMBER           },
		{EPacketPropertyType::NormalPressure,          GUID_PACKETPROPERTY_GUID_NORMAL_PRESSURE,          STR_GUID_NORMALPRESSURE         },
		{EPacketPropertyType::TangentPressure,         GUID_PACKETPROPERTY_GUID_TANGENT_PRESSURE,         STR_GUID_TANGENTPRESSURE        },
		{EPacketPropertyType::ButtonPressure,          GUID_PACKETPROPERTY_GUID_BUTTON_PRESSURE,          STR_GUID_BUTTONPRESSURE         },
		{EPacketPropertyType::XTiltOrientation,        GUID_PACKETPROPERTY_GUID_X_TILT_ORIENTATION,       STR_GUID_XTILTORIENTATION       },
		{EPacketPropertyType::YTiltOrientation,        GUID_PACKETPROPERTY_GUID_Y_TILT_ORIENTATION,       STR_GUID_YTILTORIENTATION       },
		{EPacketPropertyType::AzimuthOrientation,      GUID_PACKETPROPERTY_GUID_AZIMUTH_ORIENTATION,      STR_GUID_AZIMUTHORIENTATION     },
		{EPacketPropertyType::AltitudeOrientation,     GUID_PACKETPROPERTY_GUID_ALTITUDE_ORIENTATION,     STR_GUID_ALTITUDEORIENTATION    },
		{EPacketPropertyType::TwistOrientation,        GUID_PACKETPROPERTY_GUID_TWIST_ORIENTATION,        STR_GUID_TWISTORIENTATION       },
		{EPacketPropertyType::PitchRotation,           GUID_PACKETPROPERTY_GUID_PITCH_ROTATION,           STR_GUID_PITCHROTATION          },
		{EPacketPropertyType::RollRotation,            GUID_PACKETPROPERTY_GUID_ROLL_ROTATION,            STR_GUID_ROLLROTATION           },
		{EPacketPropertyType::YawRotation,             GUID_PACKETPROPERTY_GUID_YAW_ROTATION,             STR_GUID_YAWROTATION            },
		{EPacketPropertyType::Width,                   GUID_PACKETPROPERTY_GUID_WIDTH,                    STR_GUID_WIDTH                  },
		{EPacketPropertyType::Height,                  GUID_PACKETPROPERTY_GUID_HEIGHT,                   STR_GUID_HEIGHT                 },
		{EPacketPropertyType::FingerContactConfidence, GUID_PACKETPROPERTY_GUID_FINGERCONTACTCONFIDENCE,  STR_GUID_FINGERCONTACTCONFIDENCE},
		{EPacketPropertyType::DeviceContactID,         GUID_PACKETPROPERTY_GUID_DEVICE_CONTACT_ID,        STR_GUID_DEVICE_CONTACT_ID      }
	};
	static_assert(std::size(PacketPropertyConstants) == static_cast<uint32>(EPacketPropertyType::Num_Enumerators));
}
