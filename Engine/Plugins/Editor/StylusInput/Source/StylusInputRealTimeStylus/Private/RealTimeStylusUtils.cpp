// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealTimeStylusUtils.h"

#include <Windows/AllowWindowsPlatformAtomics.h>
	#include <comdef.h>
#include <Windows/HideWindowsPlatformAtomics.h>

#include <StylusInputUtils.h>

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::RealTimeStylus
{
	FString GetCOMErrorMessage(const HRESULT Result)
	{
		const _com_error COMError(Result);
		const LPCTSTR ErrorMessage = COMError.ErrorMessage();
		return ErrorMessage;
	}

	void LogCOMError(const FString& Preamble, const HRESULT Result)
	{
		LogError(Preamble, FString::Format(TEXT("COM Interface Error: {0}"), {GetCOMErrorMessage(Result)}));
	}
}
