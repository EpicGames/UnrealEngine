// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Containers/UnrealString.h>

namespace UE::StylusInput::RealTimeStylus
{
	void LogCOMError(const FString& Preamble, HRESULT Result);

	inline bool Succeeded(const HRESULT Result, const FString& LogPreamble)
	{
		if (Result >= 0)
		{
			return true;
		}
		LogCOMError(LogPreamble, Result);
		return false;
	}

	inline bool Failed(const HRESULT Result, const FString& LogPreamble)
	{
		if (Result >= 0)
		{
			return false;
		}
		LogCOMError(LogPreamble, Result);
		return true;
	}
}
