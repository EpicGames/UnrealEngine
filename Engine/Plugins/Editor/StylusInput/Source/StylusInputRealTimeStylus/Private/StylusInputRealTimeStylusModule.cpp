// Copyright Epic Games, Inc. All Rights Reserved.

#include <Modules/ModuleManager.h>

#include "RealTimeStylusInterface.h"

namespace UE::StylusInput::RealTimeStylus
{
	class FStylusInputRealTimeStylusModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		TUniquePtr<IStylusInputInterface> RegisteredInterface;
	};

	void FStylusInputRealTimeStylusModule::StartupModule()
	{
		RegisteredInterface = FRealTimeStylusInterface::Create();
		if (RegisteredInterface)
		{
			RegisterInterface(RegisteredInterface.Get());
		}
	}

	void FStylusInputRealTimeStylusModule::ShutdownModule()
	{
		if (RegisteredInterface)
		{
			UnregisterInterface(RegisteredInterface.Get());

			RegisteredInterface.Reset();
		}
	}
}

IMPLEMENT_MODULE(UE::StylusInput::RealTimeStylus::FStylusInputRealTimeStylusModule, StylusInputRealTimeStylus);

/*
 * TODO
 * [ ] Propagate buttons data for StylusButtonDown/StylusButtonUp
 */
