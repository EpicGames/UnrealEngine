// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StylusInputRealTimeStylus : ModuleRules
	{
		public StylusInputRealTimeStylus(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				[
					"Core",
					"Slate",
					"SlateCore",
					"StylusInput"
				]
			);
		}
	}
}