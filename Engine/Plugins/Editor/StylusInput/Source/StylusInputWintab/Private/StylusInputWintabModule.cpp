// Copyright Epic Games, Inc. All Rights Reserved.

#include <StylusInputUtils.h>
#include <Modules/ModuleManager.h>
#include <Null/NullPlatformApplicationMisc.h>

#include "WintabAPI.h"
#include "WintabInterface.h"

namespace UE::StylusInput::Wintab
{
	class FStylusInputWintabModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		TUniquePtr<IStylusInputInterface> RegisteredInterface;
	};

	void FStylusInputWintabModule::StartupModule()
	{
		if (FNullPlatformApplicationMisc::IsUsingNullApplication())
		{
			UE_LOGF(Private::LogStylusInput, Log, "Null Application (headless) detected; Wintab interface will not be initialized.");
			return;
		}
		
		if (FWintabAPI::DriverIsAvailable())
		{
			RegisteredInterface = FWintabInterface::Create();
			if (RegisteredInterface)
			{
				RegisterInterface(RegisteredInterface.Get());
			}
			else
			{
				UE_LOGF(Private::LogStylusInput, Warning, "Wintab interface initialization failed.");
			}
		}
		else
		{
			UE_LOGF(Private::LogStylusInput, Log, "Wintab driver is not available; Wintab interface will not be initialized.");
		}
	}

	void FStylusInputWintabModule::ShutdownModule()
	{
		if (RegisteredInterface)
		{
			UnregisterInterface(RegisteredInterface.Get());

			RegisteredInterface.Reset();
		}
	}
}

IMPLEMENT_MODULE(UE::StylusInput::Wintab::FStylusInputWintabModule, StylusInputWintab);

/*
 * TODO
 * [ ] Make Wintab the default (with additional editor settings?)
 * [ ] Propagate buttons data for both Wintab and Windows (PK_BUTTONS and/or StylusButtonDown/StylusButtonDown events)
 * [ ] Queue size
 * [ ] Query multiple packets at once
 * [ ] Async
 * [ ] Deal with cursors not having a physical ID
 * [ ] Allegedly, some Wintab libraries don't handle relative button presses, so maybe switch to absolute?
 */
