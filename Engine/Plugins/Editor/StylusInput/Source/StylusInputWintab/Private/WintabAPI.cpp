// Copyright Epic Games, Inc. All Rights Reserved.

#include "WintabAPI.h"

#include <StylusInputUtils.h>
#include <Windows/WindowsPlatformProcess.h>

#define LOG_PREAMBLE "WintabAPI"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Wintab
{
	FWintabAPI::FWintabAPI()
	{
		auto GetDllHandle = [&DllHandles = DllHandles](const TCHAR* DllName) -> void* {
			void* DllHandle = FWindowsPlatformProcess::GetDllHandle(DllName);
			if (DllHandle)
			{
				DllHandles.Add(DllHandle);
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Retrieved DLL handle for {0}."), {DllName}));
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get DLL handle for {0}."), {DllName}));
			}
			return DllHandle;
		};

		auto GetDllExport = []<typename FFuncType>(const TCHAR* DllName, void* DllHandle, const TCHAR* ExportName, FFuncType& ExportHandle)
		{
			void* FuncPtr = FWindowsPlatformProcess::GetDllExport(DllHandle, ExportName);
			if (FuncPtr)
			{
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Retrieved DLL export '{0}' from {1}."), {ExportName, DllName}));
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not get DLL export '{0}' in {1}."), {ExportName, DllName}));
			}
			ExportHandle = static_cast<FFuncType>(FuncPtr);
		};

		const TCHAR* User32DllName = TEXT("User32.dll");
		if (void* User32DllHandle = GetDllHandle(User32DllName))
		{
			GetDllExport(User32DllName, User32DllHandle, TEXT("GetWindowRect"), GetWindowRect);
			GetDllExport(User32DllName, User32DllHandle, TEXT("GetSystemMetrics"), GetSystemMetrics);
		}

		const TCHAR* Wintab32DllName = TEXT("Wintab32.dll");
		if (void* Wintab32DllHandle = GetDllHandle(Wintab32DllName))
		{
			GetDllExport(Wintab32DllName, Wintab32DllHandle, TEXT("WTInfoW"), WtInfo);
			GetDllExport(Wintab32DllName, Wintab32DllHandle, TEXT("WTOpenW"), WtOpen);
			GetDllExport(Wintab32DllName, Wintab32DllHandle, TEXT("WTClose"), WtClose);
			GetDllExport(Wintab32DllName, Wintab32DllHandle, TEXT("WTOverlap"), WtOverlap);
			GetDllExport(Wintab32DllName, Wintab32DllHandle, TEXT("WTPacket"), WtPacket);
		}

		LogValidityDiagnostics();

		if (IsValid())
		{
			QueryInterfaceInfo();
		}
	}

	FWintabAPI::~FWintabAPI()
	{
		for (void* DllHandle : DllHandles)
		{
			FWindowsPlatformProcess::FreeDllHandle(DllHandle);
		}
	}

	void FWintabAPI::QueryInterfaceInfo()
	{
		{
			FWintabInfoOutputBuffer OutputBuffer;
			TCHAR* const OutputBufferPtr = OutputBuffer.Allocate(WTI_INTERFACE, IFC_WINTABID);
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_WINTABID, OutputBufferPtr);
			if (0 < BytesWritten && BytesWritten <= OutputBuffer.SizeInBytes())
			{
				InterfaceInfo.ID = OutputBufferPtr;

				Log(LOG_PREAMBLE, FString::Format(TEXT("Interface ID: {0}"), {InterfaceInfo.ID}));
			}
			else if (BytesWritten > OutputBuffer.SizeInBytes())
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query tablet hardware identification string (IFC_WINTABID, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			WORD SpecVersion;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_SPECVERSION, &SpecVersion);
			if (BytesWritten == sizeof(SpecVersion))
			{
				InterfaceInfo.SpecificationVersion = {static_cast<uint16>(SpecVersion >> 8), static_cast<uint16>(SpecVersion & 0xFF)};

				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Specification Version: {0}.{1}"),
					           {InterfaceInfo.SpecificationVersion.Major, InterfaceInfo.SpecificationVersion.Minor}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query Specification Version (IFC_SPECVERSION, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			WORD ImplVersion;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_IMPLVERSION, &ImplVersion);
			if (BytesWritten == sizeof(ImplVersion))
			{
				InterfaceInfo.ImplementationVersion = {static_cast<uint16>(ImplVersion >> 8), static_cast<uint16>(ImplVersion & 0xFF)};

				Log(LOG_PREAMBLE, FString::Format(
					           TEXT("Implementation Version: {0}.{1}"),
					           {InterfaceInfo.ImplementationVersion.Major, InterfaceInfo.ImplementationVersion.Minor}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query Implementation Version (IFC_IMPLVERSION, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			UINT MaxNumCursorTypes;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_NCURSORS, &MaxNumCursorTypes);
			if (BytesWritten == sizeof(MaxNumCursorTypes))
			{
				InterfaceInfo.MaximumNumberOfCursorTypes = MaxNumCursorTypes;

				Log(LOG_PREAMBLE, FString::Format(TEXT("Maximum Number of Cursor Types: {0}"), {MaxNumCursorTypes}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query Maximum Number of Cursor Types (IFC_NCURSORS, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			UINT NumContexts;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_NCONTEXTS, &NumContexts);
			if (BytesWritten == sizeof(NumContexts))
			{
				InterfaceInfo.MaximumNumberOfContexts = NumContexts;

				Log(LOG_PREAMBLE, FString::Format(TEXT("Maximum Number of Contexts: {0}"), {InterfaceInfo.MaximumNumberOfContexts}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query maximum number of supported contexts (IFC_NCONTEXTS, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			UINT ContextOptions;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_CTXOPTIONS, &ContextOptions);
			if (BytesWritten == sizeof(ContextOptions))
			{
				InterfaceInfo.bSupportsPacketMessages = ContextOptions & CXO_MESSAGES;
				InterfaceInfo.bSupportsCursorChangeMessages = ContextOptions & CXO_CSRMESSAGES;

				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports System Cursor Contexts: {0}"), {ContextOptions & CXO_SYSTEM ? TEXT("Yes") : TEXT("No")}));
				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports Pen Windows Contexts: {0}"), {ContextOptions & CXO_PEN ? TEXT("Yes") : TEXT("No")}));
				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports WT_PACKET Messages: {0}"), {ContextOptions & CXO_MESSAGES ? TEXT("Yes") : TEXT("No")}));
				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports WT_CSRCHANGE  Messages: {0}"), {ContextOptions & CXO_CSRMESSAGES ? TEXT("Yes") : TEXT("No")}));
				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports Margins: {0}"), {ContextOptions & CXO_MARGIN ? TEXT("Yes") : TEXT("No")}));
				Log(LOG_PREAMBLE, FString::Format(
					    TEXT("Supports Inside Margins: {0}"), {ContextOptions & CXO_MGNINSIDE ? TEXT("Yes") : TEXT("No")}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query supported context options (IFC_CTXOPTIONS, return value {0})."),
					         {BytesWritten}));
			}
		}

		{
			UINT NumDevices;
			const UINT BytesWritten = WtInfo(WTI_INTERFACE, IFC_NDEVICES, &NumDevices);
			if (BytesWritten == sizeof(NumDevices))
			{
				InterfaceInfo.NumberOfDevices = NumDevices;

				Log(LOG_PREAMBLE, FString::Format(TEXT("Number of Devices: {0}"), {InterfaceInfo.NumberOfDevices}));
			}
			else if (BytesWritten != 0)
			{
				LogError(LOG_PREAMBLE, FString::Format(
					         TEXT("Failed to query Number of Devices (IFC_NDEVICES, return value {0})."),
					         {BytesWritten}));
			}
		}
	}

	bool FWintabAPI::DriverIsAvailable()
	{
		bool bIsAvailable = false;

		if (void* Wintab32DllHandle = FWindowsPlatformProcess::GetDllHandle(TEXT("Wintab32.dll")))
		{
			if (FWindowsPlatformProcess::GetDllExport(Wintab32DllHandle, TEXT("WTInfoW")))
			{
				bIsAvailable = true;
			}

			FWindowsPlatformProcess::FreeDllHandle(Wintab32DllHandle);
		}

		return bIsAvailable;
	}

	FName FWintabAPI::GetName()
	{
		static FName Name("Wintab");
		return Name;
	}

	const FWintabAPI& FWintabAPI::GetInstance()
	{
		static FWintabAPI WintabAPI;
		return WintabAPI;
	}

	bool FWintabAPI::IsValid() const
	{
		bool bSuccess = true;

		bSuccess &= GetWindowRect != nullptr;
		bSuccess &= GetSystemMetrics != nullptr;
		bSuccess &= WtInfo != nullptr;
		bSuccess &= WtOpen != nullptr;
		bSuccess &= WtClose != nullptr;
		bSuccess &= WtPacket != nullptr;

		// WtOverlap is optional; missing-export diagnostics handled in LogValidityDiagnostics.

		return bSuccess;
	}

	void FWintabAPI::LogValidityDiagnostics() const
	{
		if (!WtOverlap)
		{
			LogWarning(LOG_PREAMBLE,
				"WTOverlap export is not available; multi-window stylus context promotion is disabled. "
				"Only the most recently opened tablet context will receive packets across windows.");
		}

		if (!IsValid())
		{
			FString MissingExports;
			auto Append = [&MissingExports](const TCHAR* Name, const void* Ptr)
			{
				if (!Ptr)
				{
					if (!MissingExports.IsEmpty())
					{
						MissingExports += TEXT(", ");
					}
					MissingExports += Name;
				}
			};

			Append(TEXT("GetWindowRect"), reinterpret_cast<const void*>(GetWindowRect));
			Append(TEXT("GetSystemMetrics"), reinterpret_cast<const void*>(GetSystemMetrics));
			Append(TEXT("WTInfoW"), reinterpret_cast<const void*>(WtInfo));
			Append(TEXT("WTOpenW"), reinterpret_cast<const void*>(WtOpen));
			Append(TEXT("WTClose"), reinterpret_cast<const void*>(WtClose));
			Append(TEXT("WTPacket"), reinterpret_cast<const void*>(WtPacket));

			LogError(LOG_PREAMBLE, FString::Format(
				TEXT("Wintab API initialization is incomplete; missing required DLL exports: {0}. Stylus input via Wintab will not be available."),
				{MissingExports}));
		}
	}

	void FWintabAPI::UpdateNumberOfDevices() const
	{
		UINT NumDevices;
		if (WtInfo(WTI_INTERFACE, IFC_NDEVICES, &NumDevices) == sizeof(NumDevices))
		{
			if (NumDevices != InterfaceInfo.NumberOfDevices)
			{
				InterfaceInfo.NumberOfDevices = NumDevices;

				Log(LOG_PREAMBLE, FString::Format(TEXT("Wintab Number of Devices: {0}"), {InterfaceInfo.NumberOfDevices}));
			}
		}
	}
}

#undef LOG_PREAMBLE
