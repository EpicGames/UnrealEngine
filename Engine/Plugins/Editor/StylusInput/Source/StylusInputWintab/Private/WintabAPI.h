// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Containers/Array.h>
#include <Containers/UnrealString.h>
#include <Templates/UniquePtr.h>

#include <Windows/AllowWindowsPlatformTypes.h>

	#define WIN32_LEAN_AND_MEAN
	#include <windows.h>

	#undef INT // WINTAB.H defines an INT macro that collides with a macro definition in AllowWindowsPlatformTypes.h when using Unity builds.

	#include <WINTAB.H>
	#define PACKETDATA (/*PK_CONTEXT |*/ PK_STATUS | PK_TIME | /*PK_CHANGED |*/ PK_SERIAL_NUMBER | PK_CURSOR | PK_BUTTONS | PK_X | PK_Y | PK_Z | PK_NORMAL_PRESSURE | PK_TANGENT_PRESSURE | PK_ORIENTATION | PK_ROTATION)
	#define PACKETMODE PK_BUTTONS
	#include <PKTDEF.H>

#include <Windows/HideWindowsPlatformTypes.h>

#define WTINFO_WORKAROUND_FOR_OUTPUT_BUFFER_SIZE_BUG 1

namespace UE::StylusInput::Wintab
{
	struct FWintabInterfaceInfo
	{
		struct FVersion
		{
			uint16 Major;
			uint16 Minor;
		};

		FString ID;
		FVersion SpecificationVersion = { 0, 0 };
		FVersion ImplementationVersion = { 0, 0 };
		uint32 MaximumNumberOfCursorTypes = 0;
		uint32 MaximumNumberOfContexts = 0;
		bool bSupportsPacketMessages = false;
		bool bSupportsCursorChangeMessages = false;

		mutable uint32 NumberOfDevices = 0;
	};

	class FWintabAPI
	{
	public:
		static bool DriverIsAvailable();

		static FName GetName();
		static const FWintabAPI& GetInstance();

		bool IsValid() const;

		void LogValidityDiagnostics() const;

		void UpdateNumberOfDevices() const;

		using FFuncGetWindowRect = BOOL(HWND, LPRECT);
		FFuncGetWindowRect* GetWindowRect = nullptr;

		using FFuncGetSystemMetrics = int(int);
		FFuncGetSystemMetrics* GetSystemMetrics = nullptr;

		using FFuncWtInfo = UINT(UINT, UINT, LPVOID);
		FFuncWtInfo* WtInfo = nullptr;

		using FFuncWtOpen = HCTX(HWND, LPLOGCONTEXT, BOOL);
		FFuncWtOpen* WtOpen = nullptr;

		using FFuncWtClose = BOOL(HCTX);
		FFuncWtClose* WtClose = nullptr;

		using FFuncWtOverlap = BOOL(HCTX, BOOL);
		FFuncWtOverlap* WtOverlap = nullptr;

		using FFuncWtPacket = BOOL(HCTX, UINT, LPVOID);
		FFuncWtPacket* WtPacket = nullptr;

		FWintabInterfaceInfo InterfaceInfo;

	private:

		FWintabAPI();
		~FWintabAPI();

		void QueryInterfaceInfo();

		TArray<void*> DllHandles;
	};

	inline uint32 HctxToUint32(const HCTX Hctx)
	{
		return IntCastChecked<uint32>(reinterpret_cast<uint64>(Hctx));
	}

	inline float Fix32ToFloat(const FIX32 Value)
	{
		return Value * (1.0f / 65536.0f);
	}

#if WTINFO_WORKAROUND_FOR_OUTPUT_BUFFER_SIZE_BUG

	/*
	 * Apparently, some drivers have issues with a nullptr being passed as output buffer for WtInfo() calls.
	 * The workaround is to just always use a relatively large fixed size buffer and hope for the best.
	 */

	class FWintabInfoOutputBuffer
	{
	public:
		TCHAR* Allocate(UINT, UINT)
		{
			if (!Buffer.IsValid())
			{
				Buffer.Reset(new TCHAR[Size]);
			}

			Buffer[0] = 0;

			return Buffer.Get();
		}

		static UINT SizeInBytes()
		{
			return Size * sizeof(TCHAR);
		}

	private:
		enum { Size = 8192 };
		TUniquePtr<TCHAR[]> Buffer;
	};

#else

	class FWintabInfoOutputBuffer
	{
	public:
		FWintabInfoOutputBuffer()
			: WintabAPI(FWintabAPI::GetInstance())
		{
		}

		TCHAR* Allocate(const UINT Category, const UINT Index)
		{
			// If the following call crashes, consider enabling the workaround available via WTINFO_WORKAROUND_FOR_OUTPUT_BUFFER_SIZE_BUG.
			const UINT RequiredSize = WintabAPI.WtInfo(Category, Index, nullptr)  / sizeof(TCHAR);

			if (Size < RequiredSize)
			{
				Size = RequiredSize;
				Buffer.Reset(new TCHAR[Size]);
			}

			if (Size > 0)
			{
				Buffer[0] = 0;
			}

			return Buffer.Get();
		}

		UINT SizeInBytes() const
		{
			return Size * sizeof(TCHAR);
		}

	private:
		const FWintabAPI& WintabAPI;
		TUniquePtr<TCHAR[]> Buffer;
		UINT Size = 0;
	};

#endif
}
