// Copyright Epic Games, Inc. All Rights Reserved.

#include "WintabInstance.h"

#include <StylusInput.h>
#include <StylusInputPacket.h>
#include <StylusInputUtils.h>

#include "WintabAPI.h"
#include "WintabInterface.h"

#define LOCTEXT_NAMESPACE "WintabInstance"
#define LOG_PREAMBLE "WintabInstance"

#define ENABLE_DEBUG_EVENTS_FOR_INVALID_PACKETS 0

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Wintab
{
	bool SetupWintabContext(const FWintabAPI& WintabAPI, UINT WintabContextIndex, LOGCONTEXT& WintabContext)
	{
		bool bSuccess = true;

		WintabContext.lcPktData = PACKETDATA;
		WintabContext.lcPktMode = PACKETMODE;

		// CXO_SYSTEM is intentional even though we override lcOut* below: it tells the driver to incorporate user-side control-panel mappings
		// (Wacom Mapping, Force Proportions, active-area restrictions, etc.) into lcIn* of the returned context. We then keep lcOut* matching lcIn*
		// (with Y flipped) so the driver delivers pkX/pkY in tablet-resolution units — preserving sub-pixel precision after the manual screen mapping
		// in SetPropertyX/Y. Switching to a "pure" system context (WTI_DSCTXS) would lose that precision since pkX/pkY would arrive in integer screen pixels.
		WintabContext.lcOptions |= CXO_CSRMESSAGES | CXO_MESSAGES | CXO_SYSTEM;

		AXIS AxisX = {};
		AXIS AxisY = {};
		AXIS AxisZ = {};

		bSuccess &= WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_X, &AxisX) == sizeof(AxisX);
		bSuccess &= WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_Y, &AxisY) == sizeof(AxisY);
		bSuccess &= WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_Z, &AxisZ) == sizeof(AxisZ);

		if (!bSuccess)
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to query device properties for Wintab context with index {0}."), {WintabContextIndex}));
			return false;
		}

		WintabContext.lcOutExtX = AxisX.axMax - AxisX.axMin + 1;
		WintabContext.lcOutExtY = AxisY.axMax - AxisY.axMin + 1;
		WintabContext.lcOutExtZ = AxisZ.axMax - AxisZ.axMin + 1;

		// In Wintab, the tablet origin is lower left. Move origin to upper left so that it corresponds to screen origin.
		WintabContext.lcOutExtY = -WintabContext.lcOutExtY;

		return bSuccess;
	}

	bool SetupTabletContextMetadata(const FWintabAPI& WintabAPI, UINT WintabContextIndex, HCTX WintabContextHandle,
	                                const LOGCONTEXT& WintabContext, FTabletContext& TabletContext)
	{
		bool bSuccess = true;

		TabletContext.WintabContextIndex = WintabContextIndex;
		TabletContext.WintabContextHandle = WintabContextHandle;

		TabletContext.InputRectangle = {
			WintabContext.lcOutOrgX, WintabContext.lcOutOrgY,
			WintabContext.lcOutOrgX + WintabContext.lcOutExtX, WintabContext.lcOutOrgY - WintabContext.lcOutExtY
		};

		FWintabInfoOutputBuffer OutputBuffer;

		TabletContext.Name = WintabContext.lcName;
		if (TabletContext.Name.IsEmpty())
		{
			TCHAR *const OutputBufferPtr = OutputBuffer.Allocate(WTI_DEVICES + WintabContextIndex, DVC_NAME);
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_NAME, OutputBufferPtr) <= OutputBuffer.SizeInBytes())
			{
				TabletContext.Name = OutputBufferPtr;
			}
			else
			{
				LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query device name for Wintab context with index {0}."), {WintabContextIndex}));
			}
		}

		TCHAR *const OutputBufferPtr = OutputBuffer.Allocate(WTI_DEVICES + WintabContextIndex, DVC_PNPID);
		if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_PNPID, OutputBufferPtr) <= OutputBuffer.SizeInBytes())
		{
			TabletContext.PlugAndPlayID = OutputBufferPtr;
		}
		else
		{
			TabletContext.PlugAndPlayID.Empty();
			LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query Plug and Play ID for Wintab context with index {0}."), {WintabContextIndex}));
		}

		UINT HardwareCapabilities;
		if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_HARDWARE, &HardwareCapabilities) != sizeof(HardwareCapabilities))
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to query hardware capabilities for Wintab context with index {0}."), {WintabContextIndex}));
			return false;
		}

		TabletContext.HardwareCapabilities =
			(HardwareCapabilities & HWC_INTEGRATED ? ETabletHardwareCapabilities::Integrated : ETabletHardwareCapabilities::None)
			| (HardwareCapabilities & HWC_TOUCH ? ETabletHardwareCapabilities::CursorMustTouch : ETabletHardwareCapabilities::None)
			| (HardwareCapabilities & HWC_HARDPROX ? ETabletHardwareCapabilities::HardProximity : ETabletHardwareCapabilities::None)
			| (HardwareCapabilities & HWC_PHYSID_CURSORS ? ETabletHardwareCapabilities::CursorsHavePhysicalIds : ETabletHardwareCapabilities::None);

		/* CURSORS */

		UINT FirstCursor, NumCursors;
		if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_FIRSTCSR, &FirstCursor) == sizeof(FirstCursor)
			&& WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_NCSRTYPES, &NumCursors) == sizeof(NumCursors))
		{
			TabletContext.WintabFirstCursor = FirstCursor;
			TabletContext.WintabNumCursors = NumCursors;
		}
		else
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to query cursor range for Wintab context with index {0}."), {WintabContextIndex}));
			return false;
		}

		return bSuccess;
	}

	void SetPropertyStatus(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		// CursorIsTouching is owned by ProcessPacket and set on Packet.PenStatus BEFORE this handler runs.
		// OR-in only the inversion bit so we never clobber it.
		if (WintabPacket.pkStatus & TPS_INVERT)
		{
			Packet.PenStatus = Packet.PenStatus | EPenStatus::CursorIsInverted;
		}
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyStatus)>);

	void SetPropertyTime(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		Packet.TimerTick = WintabPacket.pkTime;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyTime)>);

	void SetPropertySerialNumber(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		Packet.SerialNumber = WintabPacket.pkSerialNumber;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertySerialNumber)>);

	void SetPropertyCursor(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const uint32* CursorIDPtr = *reinterpret_cast<uint32 *const *>(Data);
		Packet.CursorID = *CursorIDPtr;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyCursor)>);

	void AssignSetPropertyCursorData(int8 *const Data, const uint32* CursorID)
	{
		*reinterpret_cast<const uint32**>(Data) = CursorID;
	}

	void SetPropertyButtons(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		// Todo 
		/*const WORD ButtonNumber = LOWORD(WintabPacket.pkButtons);
		const WORD ButtonState = HIWORD(WintabPacket.pkButtons);*/
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyButtons)>);

	void SetPropertyX(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const RECT *const * WindowRect = reinterpret_cast<const RECT *const *>(Data);
		const int32& WindowOffsetX = (*WindowRect)->left;

		const int32* DataInt = reinterpret_cast<const int32*>(Data + sizeof(WindowRect));
		const int32& TabletContextInputWidth = DataInt[0];
		const int32& VirtualScreenWidth = DataInt[1];
		const int32& VirtualOriginX = DataInt[2];

		Packet.X = static_cast<float>(WintabPacket.pkX) * VirtualScreenWidth / TabletContextInputWidth + VirtualOriginX - WindowOffsetX;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyX)>);

	void AssignSetPropertyXData(int8* const Data, const int32 TabletContextInputWidth, const int32 VirtualScreenWidth, const int32 VirtualOriginX,
	                            const RECT* WindowRect)
	{
		const RECT** DataRect = reinterpret_cast<const RECT**>(Data);
		*DataRect = WindowRect;

		int32* DataInt = reinterpret_cast<int32*>(Data + sizeof(DataRect));
		*DataInt++ = TabletContextInputWidth;
		*DataInt++ = VirtualScreenWidth;
		*DataInt = VirtualOriginX;
	}

	void SetPropertyY(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const RECT *const * WindowRect = reinterpret_cast<const RECT *const *>(Data);
		const int32& WindowOffsetY = (*WindowRect)->top;

		const int32* DataInt = reinterpret_cast<const int32*>(Data + sizeof(WindowRect));
		const int32& TabletContextInputHeight = DataInt[0];
		const int32& VirtualScreenHeight = DataInt[1];
		const int32& VirtualOriginY = DataInt[2];

		Packet.Y = static_cast<float>(WintabPacket.pkY) * VirtualScreenHeight / TabletContextInputHeight + VirtualOriginY - WindowOffsetY;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyY)>);

	void AssignSetPropertyYData(int8* const Data, const int32 TabletContextInputHeight, const int32 VirtualScreenHeight, const int32 VirtualOriginY,
	                            const RECT* WindowRect)
	{
		const RECT** DataRect = reinterpret_cast<const RECT**>(Data);
		*DataRect = WindowRect;

		int32* DataInt = reinterpret_cast<int32*>(Data + sizeof(DataRect));
		*DataInt++ = TabletContextInputHeight;
		*DataInt++ = VirtualScreenHeight;
		*DataInt = VirtualOriginY;
	}

	static_assert(sizeof(const RECT*) + sizeof(int32) * 3 <= FPacketPropertyHandler::SetPropertyDataBufferLength,
	              "FPacketPropertyHandler::SetPropertyData buffer too small for X/Y property data");

	void SetPropertyZ(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const float& InvResolution = *reinterpret_cast<const float*>(Data);

		Packet.Z = static_cast<float>(WintabPacket.pkZ) * InvResolution;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyZ)>);

	void AssignSetPropertyZData(int8 *const Data, const float Resolution)
	{
		*reinterpret_cast<float*>(Data) = 1.0f / Resolution;
	}

	void SetPropertyNormalPressure(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const float& InvExtent = *reinterpret_cast<const float*>(Data);

		Packet.NormalPressure = static_cast<float>(WintabPacket.pkNormalPressure) * InvExtent;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyNormalPressure)>);

	void AssignSetPropertyNormalPressureData(int8 *const Data, const int32 Extent)
	{
		*reinterpret_cast<float*>(Data) = 1.0f / Extent;
	}

	void SetPropertyTangentPressure(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const float& InvExtent = *reinterpret_cast<const float*>(Data);

		Packet.TangentPressure = static_cast<float>(WintabPacket.pkTangentPressure) * InvExtent;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyTangentPressure)>);

	void AssignSetPropertyTangentPressureData(int8 *const Data, const int32 Extent)
	{
		*reinterpret_cast<float*>(Data) = 1.0f / Extent;
	}

	void SetPropertyOrientation(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const float& InvScaleAzimuth = reinterpret_cast<const float*>(Data)[0];
		const float& InvScaleAltitude = reinterpret_cast<const float*>(Data)[1];
		const float& InvScaleTwist = reinterpret_cast<const float*>(Data)[2];

		Packet.AzimuthOrientation = WintabPacket.pkOrientation.orAzimuth * InvScaleAzimuth;
		Packet.AltitudeOrientation = WintabPacket.pkOrientation.orAltitude * InvScaleAltitude;
		Packet.TwistOrientation = WintabPacket.pkOrientation.orTwist * InvScaleTwist;

		const float AzimuthRadians = FMath::DegreesToRadians(Packet.AzimuthOrientation);
		const float AltitudeRadians = FMath::DegreesToRadians(Packet.AltitudeOrientation);

		const float X = FMath::Sin(AzimuthRadians) * FMath::Cos(AltitudeRadians);
		const float Y = FMath::Cos(AzimuthRadians) * FMath::Cos(AltitudeRadians);

		Packet.XTiltOrientation = FMath::RadiansToDegrees(FMath::Asin(-X));
		Packet.YTiltOrientation = FMath::RadiansToDegrees(FMath::Asin(-Y));
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyOrientation)>);

	void AssignSetPropertyOrientationData(int8 *const Data, const float AzimuthResolution, const float AltitudeResolution, const float TwistResolution)
	{
		float* DataFloat = reinterpret_cast<float*>(Data);
		*DataFloat++ = 1.0f / AzimuthResolution * 360.0f;
		*DataFloat++ = 1.0f / AltitudeResolution * 360.0f;
		*DataFloat = 1.0f / TwistResolution * 360.0f;
	}

	void SetPropertyRotation(FStylusInputPacket& Packet, const PACKET& WintabPacket, const int8* Data)
	{
		const float& InvScalePitch = reinterpret_cast<const float*>(Data)[0];
		const float& InvScaleRoll = reinterpret_cast<const float*>(Data)[1];
		const float& InvScaleYaw = reinterpret_cast<const float*>(Data)[2];

		Packet.PitchRotation = WintabPacket.pkRotation.roPitch * InvScalePitch;
		Packet.RollRotation = WintabPacket.pkRotation.roRoll * InvScaleRoll;
		Packet.YawRotation = WintabPacket.pkRotation.roYaw * InvScaleYaw;
	}
	static_assert(std::is_same_v<FPacketPropertyHandler::FFuncSetProperty, decltype(SetPropertyRotation)>);

	void AssignSetPropertyRotationData(int8 *const Data, const float PitchResolution, const float RollResolution, const float YawResolution)
	{
		float* DataFloat = reinterpret_cast<float*>(Data);
		*DataFloat++ = 1.0f / PitchResolution * 360.0f;
		*DataFloat++ = 1.0f / RollResolution * 360.0f;
		*DataFloat = 1.0f / YawResolution * 360.0f;
	}

	bool SetupTabletContextPacketDescriptionData(const FWintabAPI& WintabAPI, UINT WintabContextIndex, const LOGCONTEXT& WintabContext,
	                                             const RECT& WindowRect, FTabletContext& TabletContext, const uint32* CursorIDPtr)
	{
		bool bSuccess = true;

		TabletContext.SupportedProperties = ETabletSupportedProperties::None;
		TabletContext.NumPacketProperties = 0;

		WTPKT DataAvailableForAllCursors;
		WTPKT DataAvailableForSomeCursors;
		bSuccess &= WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_PKTDATA, &DataAvailableForAllCursors) == sizeof(DataAvailableForAllCursors);
		bSuccess &= WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_CSRDATA, &DataAvailableForSomeCursors) == sizeof(DataAvailableForSomeCursors);

		if (!bSuccess)
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to query available packet data for Wintab context with index {0}."), {WintabContextIndex}));
			return false;
		}

		const WTPKT DataAvailable = DataAvailableForAllCursors | DataAvailableForSomeCursors;

		auto AddProperty = [&TabletContext](EPacketPropertyType Type, ETabletPropertyMetricUnit MetricUnit, int32 Minimum, int32 Maximum,
		                                    float Resolution) -> FPacketPropertyHandler&
		{
			FPacketProperty& Property = TabletContext.PacketProperties[TabletContext.NumPacketProperties];
			Property.Type = Type;
			Property.MetricUnit = MetricUnit;
			Property.Minimum = Minimum;
			Property.Maximum = Maximum;
			Property.Resolution = Resolution;

			FPacketPropertyHandler& PropertyHandler = TabletContext.PacketPropertyHandlers[TabletContext.NumPacketProperties];

			++TabletContext.NumPacketProperties;

			return PropertyHandler;
		};

		if (DataAvailable & PK_STATUS)
		{
			TabletContext.SupportedProperties = ETabletSupportedProperties::PacketStatus;
			FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Status, ETabletPropertyMetricUnit::Default, 0, 0, 0.0f);
			PropertyHandler.SetProperty = &SetPropertyStatus;
		}

		if (DataAvailable & PK_TIME)
		{
			TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::TimerTick;
			FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Time, ETabletPropertyMetricUnit::Default, 0, 0, 0.0f);
			PropertyHandler.SetProperty = &SetPropertyTime;
		}

		if (DataAvailable & PK_SERIAL_NUMBER)
		{
			TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::SerialNumber;
			FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::SerialNumber, ETabletPropertyMetricUnit::Default, 0, 0, 0.0f);
			PropertyHandler.SetProperty = &SetPropertySerialNumber;
		}

		if (DataAvailable & PK_CURSOR)
		{
			FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Cursor, ETabletPropertyMetricUnit::Default, 0, 0, 0.0f);
			PropertyHandler.SetProperty = &SetPropertyCursor;
			AssignSetPropertyCursorData(PropertyHandler.SetPropertyData, CursorIDPtr);
		}

		if (DataAvailable & PK_BUTTONS)
		{
			FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Buttons, ETabletPropertyMetricUnit::Default, 0, 0, 0.0f);
			PropertyHandler.SetProperty = &SetPropertyButtons;
		}

		if (DataAvailable & PK_X)
		{
			AXIS Axis;
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_X, &Axis) == sizeof(Axis))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::X;
				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::X, static_cast<ETabletPropertyMetricUnit>(Axis.axUnits), Axis.axMin,
				                                                      Axis.axMax, Fix32ToFloat(Axis.axResolution));
				PropertyHandler.SetProperty = &SetPropertyX;

				// Absorb lcInOrgX into the cached origin so SetPropertyX can keep its simple `pkX * scale + origin - WindowOffset` form even when the driver
				// reports a non-zero input origin (rare in practice; defensive against exotic Wintab implementations). Intermediate stays well within int32
				// for any realistic combination of operands (worst-case ~100000 * 32768 << INT32_MAX).
				const int32 VirtualScreenWidth = WintabAPI.GetSystemMetrics(SM_CXVIRTUALSCREEN);
				const int32 VirtualOriginX = WintabAPI.GetSystemMetrics(SM_XVIRTUALSCREEN);
				const int32 AdjustedVirtualOriginX = VirtualOriginX - WintabContext.lcInOrgX * VirtualScreenWidth / WintabContext.lcInExtX;

				AssignSetPropertyXData(PropertyHandler.SetPropertyData, WintabContext.lcInExtX,
				                       VirtualScreenWidth, AdjustedVirtualOriginX,
				                       &WindowRect);
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_Y)
		{
			AXIS Axis;
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_Y, &Axis) == sizeof(Axis))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::Y;
				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Y, static_cast<ETabletPropertyMetricUnit>(Axis.axUnits), Axis.axMin,
				                                                      Axis.axMax, Fix32ToFloat(Axis.axResolution));
				PropertyHandler.SetProperty = &SetPropertyY;

				// See PK_X above for rationale on the lcInOrgY adjustment.
				const int32 VirtualScreenHeight = WintabAPI.GetSystemMetrics(SM_CYVIRTUALSCREEN);
				const int32 VirtualOriginY = WintabAPI.GetSystemMetrics(SM_YVIRTUALSCREEN);
				const int32 AdjustedVirtualOriginY = VirtualOriginY - WintabContext.lcInOrgY * VirtualScreenHeight / WintabContext.lcInExtY;

				AssignSetPropertyYData(PropertyHandler.SetPropertyData, WintabContext.lcInExtY,
				                       VirtualScreenHeight, AdjustedVirtualOriginY,
				                       &WindowRect);
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_Z)
		{
			AXIS Axis;
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_Z, &Axis) == sizeof(Axis))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::Z;
				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Z, static_cast<ETabletPropertyMetricUnit>(Axis.axUnits), Axis.axMin,
				                                                      Axis.axMax, Fix32ToFloat(Axis.axResolution));

				const auto& Property = TabletContext.PacketProperties[TabletContext.NumPacketProperties - 1]; 
				const float Resolution = [&Property]
				{
					switch (Property.MetricUnit)
					{
					case ETabletPropertyMetricUnit::Inches:
						return Property.Resolution * 2.54f;
					case ETabletPropertyMetricUnit::Centimeters:
						return Property.Resolution;
					default:
						return 1.0f;
					}
				}();

				PropertyHandler.SetProperty = &SetPropertyZ;
				AssignSetPropertyZData(PropertyHandler.SetPropertyData, Resolution);
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_NORMAL_PRESSURE)
		{
			AXIS Axis;
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_NPRESSURE, &Axis) == sizeof(Axis))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::NormalPressure;
				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::NormalPressure, static_cast<ETabletPropertyMetricUnit>(Axis.axUnits), Axis.axMin,
				                                        Axis.axMax, Fix32ToFloat(Axis.axResolution));
				PropertyHandler.SetProperty = &SetPropertyNormalPressure;
				AssignSetPropertyNormalPressureData(PropertyHandler.SetPropertyData, Axis.axMax - Axis.axMin + 1);
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_TANGENT_PRESSURE)
		{
			AXIS Axis;
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_TPRESSURE, &Axis) == sizeof(Axis))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::TangentPressure;
				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::TangentPressure, static_cast<ETabletPropertyMetricUnit>(Axis.axUnits), Axis.axMin,
				                                        Axis.axMax, Fix32ToFloat(Axis.axResolution));
				PropertyHandler.SetProperty = &SetPropertyTangentPressure;
				AssignSetPropertyTangentPressureData(PropertyHandler.SetPropertyData, Axis.axMax - Axis.axMin + 1);
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_ORIENTATION)
		{
			AXIS Orientation[3];
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_ORIENTATION, &Orientation) == sizeof(Orientation))
			{
				if (Orientation[0].axResolution && Orientation[1].axResolution)
				{
					TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::AzimuthOrientation |
						ETabletSupportedProperties::AltitudeOrientation | ETabletSupportedProperties::TwistOrientation |
						ETabletSupportedProperties::XTiltOrientation | ETabletSupportedProperties::YTiltOrientation;

					if (Orientation[0].axUnits != TU_CIRCLE || Orientation[1].axUnits != TU_CIRCLE || Orientation[2].axUnits != TU_CIRCLE)
					{
						LogWarning(LOG_PREAMBLE, FString::Format(
							           TEXT("Units for orientation are not all TU_CIRCLE for Wintab context with index {0}."), {WintabContextIndex}));
					}

					FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Orientation, static_cast<ETabletPropertyMetricUnit>(Orientation[0].axUnits), 0,
					                                        0, Fix32ToFloat(Orientation[0].axResolution));
					PropertyHandler.SetProperty = &SetPropertyOrientation;
					AssignSetPropertyOrientationData(PropertyHandler.SetPropertyData, Fix32ToFloat(Orientation[0].axResolution),
					                                 Fix32ToFloat(Orientation[1].axResolution),
					                                 Fix32ToFloat(Orientation[2].axResolution));
				}
				else
				{
					LogWarning(LOG_PREAMBLE, FString::Format(
						           TEXT(
							           "Orientation reported as supported but azimuth/altitude resolution is zero for Wintab context with index {0}; orientation will be unavailable."),
						           {WintabContextIndex}));
				}
			}
			else
			{
				bSuccess = false;
			}
		}

		if (DataAvailable & PK_ROTATION)
		{
			AXIS Rotation[3];
			if (WintabAPI.WtInfo(WTI_DEVICES + WintabContextIndex, DVC_ROTATION, &Rotation) == sizeof(Rotation))
			{
				TabletContext.SupportedProperties = TabletContext.SupportedProperties | ETabletSupportedProperties::PitchRotation |
					ETabletSupportedProperties::RollRotation | ETabletSupportedProperties::YawRotation;

				if (Rotation[0].axUnits != TU_CIRCLE || Rotation[1].axUnits != TU_CIRCLE || Rotation[2].axUnits != TU_CIRCLE)
				{
					LogWarning(LOG_PREAMBLE, FString::Format(
								 TEXT("Units for rotation are not all TU_CIRCLE for Wintab context with index {0}."), {WintabContextIndex}));
				}

				FPacketPropertyHandler& PropertyHandler = AddProperty(EPacketPropertyType::Rotation, static_cast<ETabletPropertyMetricUnit>(Rotation[0].axUnits), 0, 0,
				                                        Fix32ToFloat(Rotation[0].axResolution));
				PropertyHandler.SetProperty = &SetPropertyRotation;
				AssignSetPropertyRotationData(PropertyHandler.SetPropertyData, Fix32ToFloat(Rotation[0].axResolution), Fix32ToFloat(Rotation[1].axResolution),
				                              Fix32ToFloat(Rotation[2].axResolution));
			}
			else
			{
				bSuccess = false;
			}
		}

		if (!bSuccess)
		{
			LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to query packet property information for Wintab context with index {0}."), {WintabContextIndex}));
		}

		return bSuccess;
	}

	bool SetupTabletContext(const FWintabAPI& WintabAPI, UINT WintabContextIndex, HCTX WintabContextHandle, const LOGCONTEXT& WintabContext,
	                        const RECT& WindowRect, FTabletContext& TabletContext, const uint32* CursorIDPtr)
	{
		bool bSuccess = true;

		bSuccess &= SetupTabletContextMetadata(WintabAPI, WintabContextIndex, WintabContextHandle, WintabContext, TabletContext);
		bSuccess &= SetupTabletContextPacketDescriptionData(WintabAPI, WintabContextIndex, WintabContext, WindowRect, TabletContext, CursorIDPtr);

		return bSuccess;
	}

	FWintabInstance::FWintabInstance(const uint32 ID, const HWND OSWindowHandle, FWintabInterface& Interface)
		: ID(ID)
		, WintabAPI(FWintabAPI::GetInstance())
		, Interface(Interface)
		, OSWindowHandle(OSWindowHandle)
	{
		UpdateWindowRect();
		UpdateTabletContexts();
	}

	FWintabInstance::~FWintabInstance()
	{
		ClearTabletContexts();

		if (const uint32 NumInvalidPackets = PacketStats.GetNumInvalidPackets())
		{
			Log(LOG_PREAMBLE, FString::Format(
				    TEXT("Wintab instance '{0}' encountered {1} invalid packets."), {FWintabInstance::GetName().ToString(), NumInvalidPackets}));
		}

		if (const uint32 NumLostPackets = PacketStats.GetNumLostPackets())
		{
			Log(LOG_PREAMBLE, FString::Format(
				    TEXT("Wintab instance '{0}' encountered {1} lost packets."), {FWintabInstance::GetName().ToString(), NumLostPackets}));
		}
	}

	bool FWintabInstance::AddEventHandler(IStylusInputEventHandler* EventHandler, const EEventHandlerThread Thread)
	{
		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to add nullptr as event handler.");
			return false;
		}

		if (Thread == EEventHandlerThread::OnGameThread)
		{
			if (EventHandlers.Contains(EventHandler))
			{
				LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' already exists."), {EventHandler->GetName()}));
				return false;
			}

			EventHandlers.Add(EventHandler);

			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was added."), {EventHandler->GetName()}));

			return true;
		}

		if (Thread == EEventHandlerThread::Asynchronous)
		{
			// TODO Handle asynchronous thread
			LogError(LOG_PREAMBLE, "Asynchronous event handler is not supported yet.");
			return false;
		}

		return false;
	}

	bool FWintabInstance::RemoveEventHandler(IStylusInputEventHandler* EventHandler)
	{
		if (!EventHandler)
		{
			LogWarning(LOG_PREAMBLE, "Tried to remove nullptr event handler.");
			return false;
		}

		if (EventHandlers.Remove(EventHandler) > 0)
		{
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Event handler '{0}' was removed."), {EventHandler->GetName()}));
			return true;
		}

		// TODO Handle asynchronous thread

		return false;
	}

	const TSharedPtr<IStylusInputTabletContext> FWintabInstance::GetTabletContext(const uint32 TabletContextID)
	{
		return TabletContexts.Get(TabletContextID);
	}

	const TSharedPtr<IStylusInputStylusInfo> FWintabInstance::GetStylusInfo(const uint32 StylusID)
	{
		return StylusInfos.Get(StylusID);
	}

	float FWintabInstance::GetPacketsPerSecond(const EEventHandlerThread EventHandlerThread) const
	{
		return PacketStats.GetPacketsPerSecond();
	}

	FName FWintabInstance::GetInterfaceName()
	{
		return FWintabAPI::GetName();
	}

	FText FWintabInstance::GetName()
	{
		return FText::Format(LOCTEXT("Wintab", "Wintab #{0}"), ID);
	}

	bool FWintabInstance::WasInitializedSuccessfully() const
	{
		return true;
	}

	void FWintabInstance::OnPacket(HCTX TabletContextHandle, UINT SerialNumber)
	{
		if (SerialNumber != 0)
		{
			PACKET WintabPacket = {};
			if (WintabAPI.WtPacket(TabletContextHandle, SerialNumber, &WintabPacket))
			{
				const uint32 TabletContextID = HctxToUint32(TabletContextHandle);

				if (bCursorChange)
				{
					ProcessCursorChange(TabletContextID, WintabPacket);
				}

				PacketStats.NewPacket(WintabPacket.pkSerialNumber);

				const FStylusInputPacket Packet = ProcessPacket(TabletContextID, WintabPacket);

				if (Packet.Type != EPacketType::Invalid)
				{
					for (IStylusInputEventHandler* EventHandler : EventHandlers)
					{
						EventHandler->OnPacket(Packet, this);
					}
				}
			}
			else
			{
				PacketStats.InvalidPacket();
#if ENABLE_DEBUG_EVENTS_FOR_INVALID_PACKETS
				DebugEvent(FString::Format(TEXT("Failed to retrieve Wintab packet number {0}."), {SerialNumber}));
#endif
			}
		}
	}

	void FWintabInstance::OnProximity(HCTX TabletContextHandle, LPARAM LParam)
	{
		if (LOWORD(LParam) || HIWORD(LParam))
		{
			// While the Wintab documentation is somewhat confusing about what is encoded in LParam, it appears that the high-order word is effectively
			// indicating that the event is triggered by hardware proximity, and the low-order word is triggered both with and without hardware proximity
			// support. Just in case, we trigger a cursor change if either of the flags are set.

			bCursorChange = true;

#if CHECK_MESSAGE_ORDER_ASSUMPTIONS
			CursorChangeTabletContextID = HctxToUint32(TabletContextHandle);
#endif
		}
	}

	void FWintabInstance::OnCursorChange(HCTX TabletContextHandle, UINT SerialNumber)
	{
		bCursorChange = true;

#if CHECK_MESSAGE_ORDER_ASSUMPTIONS
		CursorChangeTabletContextID = HctxToUint32(TabletContextHandle);
		CursorChangeSerialNumber = SerialNumber;
#endif
	}

	void FWintabInstance::OnInfoChange()
	{
		UpdateTabletContexts();
	}

	void FWintabInstance::OnWindowActivated()
	{
		if (!WintabAPI.WtOverlap)
		{
			return;
		}

		for (int32 Index = 0, Num = TabletContexts.Num(); Index < Num; ++Index)
		{
			const HCTX WintabContextHandle = TabletContexts[Index]->WintabContextHandle;
			if (!WintabAPI.WtOverlap(WintabContextHandle, Windows::TRUE))
			{
				LogWarning(LOG_PREAMBLE, FString::Format(TEXT("WTOverlap returned FALSE for HCTX {0}"), {HctxToUint32(WintabContextHandle)}));
			}
		}
	}

	void FWintabInstance::OnWindowPosChanged()
	{
		UpdateWindowRect();
	}

	void FWintabInstance::OnDisplayChange()
	{
		UpdateTabletContexts();
	}

	void FWintabInstance::OnDpiChanged()
	{
		UpdateTabletContexts();
	}

	const FTabletContext* FWintabInstance::GetTabletContextInternal(const uint32 TabletContextID) const
	{
		return TabletContexts.Get(TabletContextID).Get();
	}

	uint32 FWintabInstance::GetStylusID(const uint32 TabletContextID, uint32 CursorIndex)
	{
		DWORD CursorPhysicalID;
		UINT CursorType;
		if (WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_PHYSID, &CursorPhysicalID) == sizeof(CursorPhysicalID) &&
			WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_TYPE, &CursorType) == sizeof(CursorType))
		{
			const uint16 MaskedCursorId = MaskCursorId(CursorType);
			const uint16 MaskedCursorType = MaskCursorType(CursorType);

			const uint64 CursorID = static_cast<uint64>(MaskedCursorId) << 32 | CursorPhysicalID;
			const TTuple<uint64, uint32>* Mapping = CursorIDToStylusIDMappings.FindByPredicate(
				[CursorID](const TTuple<uint64, uint32>& Tuple)
				{
					return Tuple.Get<0>() == CursorID;
				});

			if (!Mapping)
			{
				uint32 StylusID = CursorPhysicalID;

				// Resolve any collisions of physical IDs, which are only guaranteed to be unique within a (masked) cursor type.
				while (StylusInfos.Contains(StylusID))
				{
					++StylusID;
				}

				Mapping = &CursorIDToStylusIDMappings.Emplace_GetRef(CursorID, StylusID);

				const TSharedRef<FStylusInfo> StylusInfo = StylusInfos.Add(StylusID);

				StylusInfo->WintabPhysicalID = CursorPhysicalID;
				StylusInfo->WintabCursorType = MaskedCursorType;

				const ECursorIndexType CursorIndexType = [&TabletContexts = TabletContexts, TabletContextID, CursorIndex]
				{
					if (const TSharedPtr<FTabletContext> TabletContextPtr = TabletContexts.Get(TabletContextID))
					{
						const FTabletContext& TabletContext = *TabletContextPtr;
						if (TabletContext.WintabFirstCursor <= CursorIndex && CursorIndex < static_cast<uint8>(TabletContext.WintabFirstCursor + TabletContext.WintabNumCursors))
						{
							const int8 CursorIndexTypeInt = CursorIndex - TabletContext.WintabFirstCursor;
							if (0 <= CursorIndexTypeInt && CursorIndexTypeInt < static_cast<int8>(ECursorIndexType::Num_Enumerators))
							{
								return static_cast<ECursorIndexType>(CursorIndexTypeInt);
							}
						}
					}
					return ECursorIndexType::Invalid_Enumerator;
				}();

				// If this is the inverted side of a pen, populate the stylus data with the non-inverted side instead.
				if (CursorIndexType != ECursorIndexType::Invalid_Enumerator && CursorIsInverted(CursorIndexType))
				{
					--CursorIndex;
				}

				FWintabInfoOutputBuffer OutputBuffer;
				TCHAR* OutputBufferPtr = OutputBuffer.Allocate(WTI_CURSORS + CursorIndex, CSR_NAME);
				if (WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_NAME, OutputBufferPtr) <= OutputBuffer.SizeInBytes())
				{
					StylusInfo->Name = FString::Format(TEXT("{0} ({1})"), {OutputBufferPtr, MaskedCursorTypeToString(MaskedCursorType)});
				}
				else
				{
					LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query name for Wintab cursor with index {0}."), {CursorIndex}));
				}

				BYTE NumButtons;
				if (WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_BUTTONS, &NumButtons) == sizeof(NumButtons))
				{
					StylusInfo->Buttons.SetNum(NumButtons);

					OutputBufferPtr = OutputBuffer.Allocate(WTI_CURSORS + CursorIndex, CSR_BTNNAMES);
					if (WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_BTNNAMES, OutputBufferPtr) <= OutputBuffer.SizeInBytes())
					{
						const TCHAR* Name = OutputBufferPtr;
						int32 ButtonIndex = 0;
						while (*Name)
						{
							StylusInfo->Buttons[ButtonIndex].Name = Name;
							Name += StylusInfo->Buttons[ButtonIndex].Name.Len() + 1;
							++ButtonIndex;
						}
					}
					else
					{
						LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query button names for Wintab cursor with index {0}."), {CursorIndex}));
					}

					BYTE ButtonMap[32] = {};
					if (WintabAPI.WtInfo(WTI_CURSORS + CursorIndex, CSR_BUTTONMAP, &ButtonMap) == sizeof(ButtonMap))
					{
						for (int32 ButtonIndex = 0; ButtonIndex < NumButtons; ++ButtonIndex)
						{
							// Todo
						}
					}
					else
					{
						LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query button map for Wintab cursor with index {0}."), {CursorIndex}));
					}
				}
				else
				{
					LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query number of buttons for Wintab cursor with index {0}."), {CursorIndex}));
				}
			}

			// Make sure a StylusInfo with this ID exists, and the cursor type and physical ID match.
			checkSlow(Mapping && StylusInfos.Contains(Mapping->Get<1>()) && StylusInfos.Get(Mapping->Get<1>()).IsValid()
				&& StylusInfos.Get(Mapping->Get<1>())->WintabCursorType == MaskedCursorType
				&& StylusInfos.Get(Mapping->Get<1>())->WintabPhysicalID == CursorPhysicalID);

			return Mapping->Get<1>();
		}
		else
		{
			LogWarning(LOG_PREAMBLE, FString::Format(TEXT("Failed to query cursor metadata for cursor index {0}."), {CursorIndex}));
		}

		return 0;
	}

	void FWintabInstance::ClearTabletContexts()
	{
		for (int32 Index = 0, NumIndices = TabletContexts.Num(); Index < NumIndices; ++Index)
		{
			const HCTX WintabContextHandle = TabletContexts[Index]->WintabContextHandle;

			Interface.UnregisterTabletContext(WintabContextHandle);

			if (WintabAPI.WtClose(WintabContextHandle))
			{
				LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Closed Wintab context with handle {0}."),
														 {HctxToUint32(WintabContextHandle)}));
			}
			else
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not close Wintab context with handle {0}."),
													   {HctxToUint32(WintabContextHandle)}));
			}
		}

		TabletContexts.Clear();
	}

	void FWintabInstance::UpdateTabletContexts()
	{
		ClearTabletContexts();

		UINT WintabContextIndex = 0;
		LOGCONTEXTW WintabContext;

		while (WintabAPI.WtInfo(WTI_DDCTXS + WintabContextIndex, 0, &WintabContext) == sizeof(WintabContext))
		{
			if (!SetupWintabContext(WintabAPI, WintabContextIndex, WintabContext))
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to setup Wintab context with index {0}."), {WintabContextIndex}));
				++WintabContextIndex;
				continue;
			}

			const HCTX WintabContextHandle = WintabAPI.WtOpen(OSWindowHandle, &WintabContext, Windows::TRUE);
			if (!WintabContextHandle)
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed to open Wintab context with index {0}."), {WintabContextIndex}));
				++WintabContextIndex;
				continue;
			}

			Interface.RegisterTabletContext(WintabContextHandle, this);

			const uint32 TabletContextID = HctxToUint32(WintabContextHandle);

			TSharedRef<FTabletContext> TabletContextPtr = [&TabletContexts = TabletContexts, TabletContextID]
			{
				if (TabletContexts.Contains(TabletContextID))
				{
					return TabletContexts.Get(TabletContextID).ToSharedRef();
				}
				return TabletContexts.Add(TabletContextID);
			}();

			FTabletContext& TabletContext = *TabletContextPtr;

			if (!SetupTabletContext(WintabAPI, WintabContextIndex, WintabContextHandle, WintabContext, WindowRect, TabletContext, &CurrentStylusID))
			{
				LogError(LOG_PREAMBLE, FString::Format(TEXT("Failed setting up tablet context with ID {0}."), {TabletContextID}));

				TabletContexts.Remove(TabletContextID);

				Interface.UnregisterTabletContext(WintabContextHandle);

				if (!WintabAPI.WtClose(WintabContextHandle))
				{
					LogError(LOG_PREAMBLE, FString::Format(TEXT("Could not close Wintab context with handle {0}."), {HctxToUint32(WintabContextHandle)}));
				}

				++WintabContextIndex;
				continue;
			}

			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Added tablet context for ID {0} [{1}, {2}]."), {
				                                         TabletContext.ID, TabletContext.Name , TabletContext.PlugAndPlayID
			                                         }));

			++WintabContextIndex;
		}
	}

	void FWintabInstance::UpdateWindowRect()
	{
		if (!ensure(WintabAPI.GetWindowRect(OSWindowHandle, &WindowRect)))
		{
			LogError(LOG_PREAMBLE, "Could not retrieve window rectangle; coordinates mapping will be incorrect!");
		}
	}

	void FWintabInstance::ProcessCursorChange(uint32 TabletContextID, const PACKET& WintabPacket)
	{
		check(bCursorChange);

#if CHECK_MESSAGE_ORDER_ASSUMPTIONS
		if (CursorChangeTabletContextID != TabletContextID || !(CursorChangeSerialNumber == 0 || CursorChangeSerialNumber == WintabPacket.pkSerialNumber))
		{
			DebugEvent("Cursor change assumptions violated: the next packet does not match the previous WT_PROXIMITY/WT_CSRCHANGE messages.");
		}
#endif

		CurrentStylusID = GetStylusID(TabletContextID, WintabPacket.pkCursor);

		bCursorChange = false;

#if CHECK_MESSAGE_ORDER_ASSUMPTIONS
		CursorChangeTabletContextID = 0;
		CursorChangeSerialNumber = 0;
#endif
	}

	FStylusInputPacket FWintabInstance::ProcessPacket(uint32 TabletContextID, const PACKET& WintabPacket)
	{
		FStylusInputPacket Packet{TabletContextID};

		if (const FTabletContext* TabletContext = GetTabletContextInternal(TabletContextID))
		{
			const ETabletSupportedProperties Supported = TabletContext->SupportedProperties;

			// Contact is derived from tip pressure. Devices without a pressure axis fall back to pkZ proximity height.
			// Read the raw packet field: Packet.NormalPressure is not populated until the property loop below.
			const bool bPacketOnDigitizer = static_cast<bool>(Supported & ETabletSupportedProperties::NormalPressure)
				? WintabPacket.pkNormalPressure > 0
				: !static_cast<bool>(Supported & ETabletSupportedProperties::Z) || WintabPacket.pkZ == 0;

			Packet.Type = bPacketOnDigitizer
							  ? bLastPacketOnDigitizer ? EPacketType::OnDigitizer : EPacketType::StylusDown
							  : bLastPacketOnDigitizer ? EPacketType::StylusUp : EPacketType::AboveDigitizer;
			bLastPacketOnDigitizer = bPacketOnDigitizer;

			// PenStatus is primarily based on just determined contact. SetPropertyStatus called in the loop below OR-contributes only the CursorIsInverted bit.
			Packet.PenStatus = bPacketOnDigitizer ? EPenStatus::CursorIsTouching : EPenStatus::None;

			for (int32 Index = 0, Num = TabletContext->NumPacketProperties; Index < Num; ++Index)
			{
				const FPacketPropertyHandler& PropertyHandler = TabletContext->PacketPropertyHandlers[Index];
				PropertyHandler.SetProperty(Packet, WintabPacket, PropertyHandler.SetPropertyData);
			}
		}

		return Packet;
	}

	void FWintabInstance::DebugEvent(const FString& Message)
	{
		for (IStylusInputEventHandler* EventHandler : EventHandlers)
		{
			EventHandler->OnDebugEvent(Message, this);
		}
	}
}

#undef LOG_PREAMBLE
#undef LOCTEXT_NAMESPACE
