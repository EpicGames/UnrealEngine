// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <Containers/Array.h>
#include <Microsoft/MinimalWindowsApi.h>

#include "WintabTabletContext.h"
#include "WintabStats.h"
#include "WintabStylus.h"

#define CHECK_MESSAGE_ORDER_ASSUMPTIONS NDEBUG

namespace UE::StylusInput
{
	class IStylusInputEventHandler;
}

namespace UE::StylusInput::Wintab
{
	class FWintabInterface;
	class FWintabAPI;

	class FWintabInstance : public IStylusInputInstance
	{
	public:
		explicit FWintabInstance(uint32 ID, HWND OSWindowHandle, FWintabInterface& Interface);
		virtual ~FWintabInstance() override;

		virtual bool AddEventHandler(IStylusInputEventHandler* EventHandler, EEventHandlerThread Thread) override;
		virtual bool RemoveEventHandler(IStylusInputEventHandler* EventHandler) override;

		virtual const TSharedPtr<IStylusInputTabletContext> GetTabletContext(uint32 TabletContextID) override;
		virtual const TSharedPtr<IStylusInputStylusInfo> GetStylusInfo(uint32 StylusID) override;

		virtual float GetPacketsPerSecond(EEventHandlerThread EventHandlerThread) const override;

		virtual FName GetInterfaceName() override;
		virtual FText GetName() override;

		virtual bool WasInitializedSuccessfully() const override;

		HWND GetOSWindowHandle() const { return OSWindowHandle; }

		void OnPacket(HCTX TabletContextHandle, UINT SerialNumber);
		void OnProximity(HCTX TabletContextHandle, LPARAM LParam);
		void OnCursorChange(HCTX TabletContextHandle, UINT SerialNumber);
		void OnInfoChange();
		void OnWindowActivated();
		void OnWindowPosChanged();
		void OnDisplayChange();
		void OnDpiChanged();

	private:
		const FTabletContext* GetTabletContextInternal(uint32 TabletContextID) const;
		uint32 GetStylusID(uint32 TabletContextID, uint32 CursorIndex);

		void ClearTabletContexts();
		void UpdateTabletContexts();
		void UpdateWindowRect();

		void ProcessCursorChange(uint32 TabletContextID, const PACKET& WintabPacket);
		FStylusInputPacket ProcessPacket(uint32 TabletContextID, const PACKET& WintabPacket);

		void DebugEvent(const FString& Message);

		const uint32 ID;
		const FWintabAPI& WintabAPI;
		FWintabInterface& Interface;
		const HWND OSWindowHandle;
		RECT WindowRect = {};

		FTabletContextContainer TabletContexts;
		FStylusInfoContainer StylusInfos;

		TArray<IStylusInputEventHandler*> EventHandlers;
		FPacketStats PacketStats;

		uint32 CurrentStylusID = 0;
		TArray<TTuple<uint64, uint32>> CursorIDToStylusIDMappings;

		bool bCursorChange = false;
		bool bLastPacketOnDigitizer = false;

#if CHECK_MESSAGE_ORDER_ASSUMPTIONS
		uint32 CursorChangeTabletContextID = 0;
		uint32 CursorChangeSerialNumber = 0;
#endif
	};
}
