// Copyright Epic Games, Inc. All Rights Reserved.

#include "WintabInterface.h"

#include <StylusInput.h>
#include <StylusInputUtils.h>

#include <GenericPlatform/GenericWindow.h>
#include <Framework/Application/SlateApplication.h>
#include <Templates/SharedPointer.h>
#include <Widgets/SWindow.h>
#include <Windows/WindowsApplication.h>

#include "WintabInstance.h"
#include "WintabAPI.h"

#define LOCTEXT_NAMESPACE "WintabInterface"

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Wintab
{
	FWintabInterface::FWintabInterface()
		: MessageHandler(*this)
	{
		if (FSlateApplication::IsInitialized())
		{
			if (const TSharedPtr<GenericApplication> WindowsApplication = FSlateApplication::Get().GetPlatformApplication())
			{
				static_cast<FWindowsApplication*>(WindowsApplication.Get())->AddMessageHandler(MessageHandler);
			}
		}
	}

	FWintabInterface::~FWintabInterface()
	{
		if (FSlateApplication::IsInitialized())
		{
			if (const TSharedPtr<GenericApplication> WindowsApplication = FSlateApplication::Get().GetPlatformApplication())
			{
				static_cast<FWindowsApplication*>(WindowsApplication.Get())->RemoveMessageHandler(MessageHandler);
			}
		}
	}

	FName FWintabInterface::GetName() const
	{
		return FWintabAPI::GetName();
	}

	IStylusInputInstance* FWintabInterface::CreateInstance(SWindow& Window)
	{
		if (FRefCountedInstance* ExistingRefCountedInstance = Instances.Find(&Window))
		{
			++ExistingRefCountedInstance->RefCount;
			return ExistingRefCountedInstance->Instance.Get();
		}

		HWND OSWindowHandle = [&Window]
		{
			const TSharedPtr<const FGenericWindow> NativeWindow = Window.GetNativeWindow();
			return NativeWindow.IsValid() ? static_cast<HWND>(NativeWindow->GetOSWindowHandle()) : nullptr;
		}();

		if (!OSWindowHandle)
		{
			LogError("WintabInterface", "Could not get native window handle.");
			return nullptr;
		}

		FWintabInstance* NewInstance = Instances.Emplace(
			&Window, {MakeUnique<FWintabInstance>(NextInstanceID++, OSWindowHandle, *this), 1}).Instance.Get();
		if (!ensureMsgf(NewInstance, TEXT("WintabInterface: Failed to create stylus input instance.")))
		{
			Instances.Remove(&Window);
			return nullptr;
		}

		InstancesByHwnd.Add(OSWindowHandle, NewInstance);

		return NewInstance;
	}

	bool FWintabInterface::ReleaseInstance(IStylusInputInstance* Instance)
	{
		check(Instance);

		FWintabInstance *const WindowsInstance = static_cast<FWintabInstance*>(Instance);

		// Find existing instance
		for (TTuple<SWindow*, FRefCountedInstance>& Entry : Instances)
		{
			FRefCountedInstance& RefCountedInstance = Entry.Get<1>();

			if (RefCountedInstance.Instance.Get() == WindowsInstance)
			{
				// Decrease reference count
				check(RefCountedInstance.RefCount > 0);
				if (--RefCountedInstance.RefCount == 0)
				{
					// Delete if there are no references left
					if (FWintabInstance* InstanceToBeRemoved = RefCountedInstance.Instance.Get())
					{
						InstancesByHwnd.Remove(InstanceToBeRemoved->GetOSWindowHandle());
					}
					Instances.Remove(Entry.Key);
				}

				return true;
			}
		}

		ensureMsgf(false, TEXT("WintabInterface: Failed to find provided instance."));
		return false;
	}

	void FWintabInterface::RegisterTabletContext(HCTX Handle, FWintabInstance* Instance)
	{
		check(IsInGameThread());
		InstancesByHctx.Add(Handle, Instance);
	}

	void FWintabInterface::UnregisterTabletContext(HCTX Handle)
	{
		check(IsInGameThread());
		InstancesByHctx.Remove(Handle);
	}

	FWintabInstance* FWintabInterface::FindInstanceByHctx(HCTX Handle) const
	{
		FWintabInstance* const* Instance = InstancesByHctx.Find(Handle);
		return Instance ? *Instance : nullptr;
	}

	FWintabInstance* FWintabInterface::FindInstanceByHwnd(HWND Window) const
	{
		FWintabInstance* const* Instance = InstancesByHwnd.Find(Window);
		return Instance ? *Instance : nullptr;
	}

	TUniquePtr<IStylusInputInterface> FWintabInterface::Create()
	{
		if (FWintabAPI::GetInstance().IsValid())
		{
			TUniquePtr<FWintabInterface> WintabInterface = MakeUnique<FWintabInterface>();
			return TUniquePtr<IStylusInputInterface>(MoveTemp(WintabInterface));
		}

		return nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
