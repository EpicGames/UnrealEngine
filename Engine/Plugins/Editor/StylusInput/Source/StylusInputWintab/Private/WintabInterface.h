// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInput.h>
#include <StylusInputInterface.h>
#include <Containers/Map.h>
#include <Templates/UniquePtr.h>

#include "WintabAPI.h"
#include "WintabMessageHandler.h"

namespace UE::StylusInput::Wintab
{
	class FWintabInstance;

	class FWintabInterface : public IStylusInputInterface
	{
	public:
		FWintabInterface();
		virtual ~FWintabInterface();

		virtual FName GetName() const override;

		virtual IStylusInputInstance* CreateInstance(SWindow& Window) override;
		virtual bool ReleaseInstance(IStylusInputInstance* Instance) override;

		void RegisterTabletContext(HCTX Handle, FWintabInstance* Instance);
		void UnregisterTabletContext(HCTX Handle);

		FWintabInstance* FindInstanceByHctx(HCTX Handle) const;
		FWintabInstance* FindInstanceByHwnd(HWND Window) const;

		template <typename FuncType>
		void ForEachInstance(FuncType&& Func) const
		{
			for (const TTuple<SWindow*, FRefCountedInstance>& Entry : Instances)
			{
				if (FWintabInstance* InstancePtr = Entry.Value.Instance.Get())
				{
					Func(*InstancePtr);
				}
			}
		}

	private:
		friend class FStylusInputWintabModule;

		static TUniquePtr<IStylusInputInterface> Create();

		struct FRefCountedInstance
		{
			TUniquePtr<FWintabInstance> Instance;
			int32 RefCount;
		};

		FWintabMessageHandler MessageHandler;

		TMap<HWND, FWintabInstance*> InstancesByHwnd;
		TMap<HCTX, FWintabInstance*> InstancesByHctx;

		TMap<SWindow*, FRefCountedInstance> Instances;
		uint32 NextInstanceID = 0;
	};
}
