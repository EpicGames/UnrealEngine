// Copyright Epic Games, Inc. All Rights Reserved.

#include "WintabMessageHandler.h"

#include <StylusInputUtils.h>

#include "WintabAPI.h"
#include "WintabInstance.h"
#include "WintabInterface.h"

#define LOG_PREAMBLE "WintabMessageHandler"

#define ENABLE_LOG_FOR_PACKET_MESSAGES 0
#define ENABLE_LOG_FOR_CONTEXT_OPEN_CLOSE_UPDATE_MESSAGES 0
#define ENABLE_LOG_FOR_CONTEXT_OVERLAP_MESSAGES 0
#define ENABLE_LOG_FOR_PROXIMITY_CSRCHANGE_MESSAGES 0
#define ENABLE_LOG_FOR_INFOCHANGE_MESSAGES 0
#define ENABLE_LOG_FOR_PACKETEXT_MESSAGES 0
#define ENABLE_LOG_FOR_INVALID_PACKETS 0

using namespace UE::StylusInput::Private;

namespace UE::StylusInput::Wintab
{
FWintabMessageHandler::FWintabMessageHandler(FWintabInterface& Interface)
	: Interface(Interface)
	, WintabAPI(FWintabAPI::GetInstance())
{
}

bool FWintabMessageHandler::ProcessMessage(const HWND Hwnd, const uint32 Msg, const WPARAM WParam, const LPARAM LParam, int32& OutResult)
{
	bool bHandled = false;

	switch (Msg)
	{
	case WT_PACKET:
		{
#if ENABLE_LOG_FOR_PACKET_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(
						   TEXT("Received WT_PACKET(SerialNumber={0}, Context={1}) message."), {static_cast<uint32>(WParam), static_cast<uint32>(LParam)}));
#endif
			if (FWintabInstance* Instance = Interface.FindInstanceByHctx(reinterpret_cast<HCTX>(LParam)))
			{
				Instance->OnPacket(reinterpret_cast<HCTX>(LParam), static_cast<UINT>(WParam));
			}
			bHandled = true;
			break;
		}

	case WT_CTXOPEN:
		{
#if ENABLE_LOG_FOR_CONTEXT_OPEN_CLOSE_UPDATE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Received WT_CTXOPEN message (Context={0}, Status={1})."), {WParam, LParam}));
#endif
			bHandled = true;
			break;
		}

	case WT_CTXCLOSE:
		{
#if ENABLE_LOG_FOR_CONTEXT_OPEN_CLOSE_UPDATE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Received WT_CTXCLOSE message (Context={0}, Status={1})."), {WParam, LParam}));
#endif
			bHandled = true;
			break;
		}

	case WT_CTXUPDATE:
		{
#if ENABLE_LOG_FOR_CONTEXT_OPEN_CLOSE_UPDATE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Received WT_CTXUPDATE message (Context={0}, Status={1})."), {WParam, LParam}));
#endif
			bHandled = true;
			break;
		}

	case WT_CTXOVERLAP:
		{
#if ENABLE_LOG_FOR_CONTEXT_OVERLAP_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(TEXT("Received WT_CTXOVERLAP message (Context={0}, Status={1})."), {WParam, LParam}));
#endif
			bHandled = true;
			break;
		}

	case WT_PROXIMITY:
		{
#if ENABLE_LOG_FOR_PROXIMITY_CSRCHANGE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(
						   TEXT("Received WT_PROXIMITY message (Context={0}, ContextEnter={1}, ProximityEnter={2})."), {
							   static_cast<uint32>(WParam), LOWORD(LParam), HIWORD(LParam)
						   }));
#endif
			if (FWintabInstance* Instance = Interface.FindInstanceByHctx(reinterpret_cast<HCTX>(WParam)))
			{
				Instance->OnProximity(reinterpret_cast<HCTX>(WParam), LParam);
			}
			bHandled = true;
			break;
		}

	case WT_INFOCHANGE:
		{
#if ENABLE_LOG_FOR_INFOCHANGE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(
						   TEXT("Received WT_INFOCHANGE(Context={0}, Category={1}, Index={2}) message."), {WParam, LOWORD(LParam), HIWORD(LParam)}));
#endif
			WintabAPI.UpdateNumberOfDevices();
			Interface.ForEachInstance([](FWintabInstance& Inst) { Inst.OnInfoChange(); });
			bHandled = true;
			break;
		}

	case WT_CSRCHANGE:
		{
#if ENABLE_LOG_FOR_PROXIMITY_CSRCHANGE_MESSAGES
			LogVerbose(LOG_PREAMBLE, FString::Format(
						   TEXT("Received WT_CSRCHANGE message (SerialNumber={0}, Context={1})."), {static_cast<uint32>(WParam), static_cast<uint32>(LParam)}));
#endif
			if (FWintabInstance* Instance = Interface.FindInstanceByHctx(reinterpret_cast<HCTX>(LParam)))
			{
				Instance->OnCursorChange(reinterpret_cast<HCTX>(LParam), static_cast<uint32>(WParam));
			}
			bHandled = true;
			break;
		}

	case WT_PACKETEXT:
		{
#if ENABLE_LOG_FOR_PACKETEXT_MESSAGES
			LogVerbose(LOG_PREAMBLE, "Received WT_PACKETEXT message.");
#endif
			bHandled = true;
			break;
		}

	// WM_* cases below intentionally do not set bHandled = true: these are standard Windows messages that Slate (and the rest of the app) still needs to
	// process for focus tracking, window management, and DPI auto-resize. We piggyback on them for stylus context bookkeeping without claiming ownership
	// of the message.

	case WM_ACTIVATE:
		{
			if (LOWORD(WParam) != WA_INACTIVE)
			{
				if (FWintabInstance* Instance = Interface.FindInstanceByHwnd(Hwnd))
				{
					Instance->OnWindowActivated();
				}
			}
			break;
		}

	case WM_WINDOWPOSCHANGED:
		{
			const WINDOWPOS* WindowPos = reinterpret_cast<const WINDOWPOS*>(LParam);
			if (WindowPos && (WindowPos->flags & (SWP_NOMOVE | SWP_NOSIZE)) != (SWP_NOMOVE | SWP_NOSIZE))
			{
				if (FWintabInstance* Instance = Interface.FindInstanceByHwnd(Hwnd))
				{
					Instance->OnWindowPosChanged();
				}
			}
			break;
		}

	case WM_DISPLAYCHANGE:
		{
			Interface.ForEachInstance([](FWintabInstance& Inst) { Inst.OnDisplayChange(); });
			break;
		}

	case WM_DPICHANGED:
		{
			if (FWintabInstance* Instance = Interface.FindInstanceByHwnd(Hwnd))
			{
				Instance->OnDpiChanged();
			}
			break;
		}

	default:
		;
	}

	return bHandled;
}
}
