// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Windows/WindowsApplication.h>

namespace UE::StylusInput::Wintab
{
	class FWintabAPI;
	class FWintabInterface;

	class FWintabMessageHandler : public IWindowsMessageHandler
	{
	public:
		FWintabMessageHandler(FWintabInterface& Interface);
		virtual ~FWintabMessageHandler() = default;

		virtual bool ProcessMessage(HWND Hwnd, uint32 Msg, WPARAM WParam, LPARAM LParam, int32& OutResult) override;

	private:
		FWintabInterface& Interface;
		const FWintabAPI& WintabAPI;
	};
}
