// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Windows/WindowsPlatformTime.h>

namespace UE::StylusInput::Wintab
{
	class FPacketStats
	{
	public:
		void NewPacket(uint32 SerialNumber)
		{
			const uint64 Timestamp = FPlatformTime::Cycles64();

			if (LatestTimestamp - EarliestTimestamp >= CyclesPerSecond)
			{
				PacketsPerSecond = EarliestTimestamp != LatestTimestamp ? NumPackets / ((LatestTimestamp - EarliestTimestamp) / CyclesPerSecond) : 0.0f;
				EarliestTimestamp = Timestamp;
				NumPackets = -1;
			}
			LatestTimestamp = Timestamp;

			NumLostPackets += SerialNumber > LatestSerialNumber ? SerialNumber - LatestSerialNumber - 1 : 0;
			LatestSerialNumber = SerialNumber;

			++NumPackets;
		}

		void InvalidPacket()
		{
			++NumInvalidPackets;
		}

		float GetPacketsPerSecond() const
		{
			const uint64 Timestamp = FPlatformTime::Cycles64();
			return Timestamp - LatestTimestamp <= CyclesPerSecond ? PacketsPerSecond : 0.0f;
		}

		uint32 GetNumInvalidPackets() const
		{
			return NumInvalidPackets;
		}

		uint32 GetNumLostPackets() const
		{
			return NumLostPackets;
		}

	private:
		uint64 EarliestTimestamp = 0;
		uint64 LatestTimestamp = 0;
		uint32 LatestSerialNumber = -1;
		uint32 NumPackets = 0;
		uint32 NumInvalidPackets = 0;
		uint32 NumLostPackets = 0;
		float PacketsPerSecond = 0.0f;
		const double CyclesPerSecond = 1.0 / FPlatformTime::GetSecondsPerCycle64();
	};
}
