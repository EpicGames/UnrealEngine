// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputTabletContext.h>
#include <StylusInputUtils.h>

namespace UE::StylusInput::Wintab
{
	enum class ECursorType : uint16
	{
		GeneralStylus = 0x0802,
		Airbrush = 0x0902,
		ArtPen = 0x0804,
		FourDMouse = 0x0004,
		FiveButtonPuck = 0x0006,

		CursorTypeMask = 0x0F06,	// Use this to mask any values coming from the Wintab API to get a valid cursor type.
		CursorIdMask = 0xFF6		// Use this to mask any values coming from the Wintab API to differentiate between different devices, incl. colors. 
	};

	inline uint16 MaskCursorType(const UINT CursorType)
	{
		return static_cast<uint16>(CursorType) & static_cast<uint16>(ECursorType::CursorTypeMask);
	}

	inline uint16 MaskCursorId(const UINT CursorType)
	{
		return static_cast<uint16>(CursorType) & static_cast<uint16>(ECursorType::CursorIdMask);
	}

	inline FString MaskedCursorTypeToString(uint16 MaskedCursorType)
	{
		switch (static_cast<ECursorType>(MaskedCursorType))
		{
		case ECursorType::GeneralStylus:
			return TEXT("General Stylus");
		case ECursorType::Airbrush:
			return TEXT("Airbrush");
		case ECursorType::ArtPen:
			return TEXT("Art Pen");
		case ECursorType::FourDMouse:
			return TEXT("4D Mouse");
		case ECursorType::FiveButtonPuck:
			return TEXT("5 Button Puck");
		default:
			return TEXT("Unknown Type");
		}
	}

	enum class ECursorIndexType : int8
	{
		PuckDevice1 = 0,
		StylusDevice1 = 1,
		InvertedStylusDevice1 = 2,
		PuckDevice2 = 3,
		StylusDevice2 = 4,
		InvertedStylusDevice2 = 5,

		Num_Enumerators,
		Invalid_Enumerator = INDEX_NONE
	};

	inline bool CursorIsInverted(ECursorIndexType Type)
	{
		return Type == ECursorIndexType::InvertedStylusDevice1 || Type == ECursorIndexType::InvertedStylusDevice2;
	}

	class FStylusButton final : public IStylusInputStylusButton
	{
	public:
		virtual FString GetID() const override { return ID; }
		virtual FString GetName() const override { return Name; }

		FString ID;
		FString Name;
	};

	class FStylusInfo final : public IStylusInputStylusInfo
	{
	public:
		FStylusInfo() = default;

		explicit FStylusInfo(const uint32 ID)
			: ID(ID)
		{
		}

		virtual uint32 GetID() const override { return ID; }
		virtual FString GetName() const override { return Name; }
		virtual uint32 GetNumButtons() const override { return Buttons.Num(); }

		virtual const IStylusInputStylusButton* GetButton(int32 Index) const override
		{
			return 0 <= Index && Index < Buttons.Num() ? reinterpret_cast<const IStylusInputStylusButton*>(&Buttons[Index]) : nullptr;
		}

		uint32 ID = 0; // This will almost always be the physical ID, unless there is a rare collision between physical IDs of cursors of different types.
		FString Name;
		TArray<FStylusButton> Buttons;

		uint32 WintabPhysicalID = -1;
		uint16 WintabCursorType = -1;
	};

	using FStylusInfoContainer = Private::TSharedRefDataContainer<FStylusInfo, false>;
	using FStylusInfoThreadSafeContainer = Private::TSharedRefDataContainer<FStylusInfo, false>;
}
