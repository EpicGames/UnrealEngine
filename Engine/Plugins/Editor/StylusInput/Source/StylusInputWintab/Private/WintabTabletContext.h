// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <StylusInputPacket.h>
#include <StylusInputTabletContext.h>
#include <StylusInputUtils.h>
#include <Math/IntRect.h>

#include "WintabAPI.h"

namespace UE::StylusInput::Wintab
{
	enum class EPacketPropertyType : int8
	{
		Status = 0,
		Time,
		Changed,
		SerialNumber,
		Cursor,
		Buttons,
		X,
		Y,
		Z,
		NormalPressure,
		TangentPressure,
		Orientation,
		Rotation,

		Num_Enumerators, // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO QUERY HOW MANY PACKET PROPERTIES THERE ARE.
		Invalid_Enumerator = INDEX_NONE // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO INDICATE THAT A PACKET PROPERTY ENTRY IS INVALID.
	};

	enum class ETabletPropertyMetricUnit : int8
	{
		Default, // Units are unknown.
		Inches,
		Centimeters,
		Circle,

		Num_Enumerators, // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO QUERY HOW MANY PACKET PROPERTIES THERE ARE.
		Invalid_Enumerator = INDEX_NONE // THIS IS NOT A VALID ENUMERATOR, BUT IT CAN BE USED TO INDICATE THAT A PACKET PROPERTY ENTRY IS INVALID.
	};

	struct FPacketProperty
	{
		EPacketPropertyType Type = EPacketPropertyType::Invalid_Enumerator;
		ETabletPropertyMetricUnit MetricUnit = ETabletPropertyMetricUnit::Default;
		int32 Minimum = 0;
		int32 Maximum = 0;
		float Resolution = 0.0f;
	};

	struct FPacketPropertyHandler
	{
		using FFuncSetProperty = void(FStylusInputPacket&, const PACKET&, const int8*);
		FFuncSetProperty* SetProperty = nullptr;

		static constexpr int32 SetPropertyDataBufferLength = 20;
		int8 SetPropertyData[SetPropertyDataBufferLength];
	};

	class FTabletContext final : public IStylusInputTabletContext
	{
	public:
		explicit FTabletContext(const uint32 ID)
			: ID(ID)
		{}

		virtual uint32 GetID() const override { return ID; }

		virtual FString GetName() const override
		{
			if (!Name.IsEmpty() && !PlugAndPlayID.IsEmpty())
			{
				return FString::Format(TEXT("{0} ({1})"), {Name, PlugAndPlayID});
			}
			return !Name.IsEmpty() ? Name : PlugAndPlayID;
		}

		virtual FIntRect GetInputRectangle() const override { return InputRectangle; }
		virtual ETabletHardwareCapabilities GetHardwareCapabilities() const override { return HardwareCapabilities; }
		virtual ETabletSupportedProperties GetSupportedProperties() const override { return SupportedProperties; }

		uint32 ID = 0;
		FString Name;
		FString PlugAndPlayID;
		FIntRect InputRectangle;
		ETabletSupportedProperties SupportedProperties = ETabletSupportedProperties::None;
		ETabletHardwareCapabilities HardwareCapabilities = ETabletHardwareCapabilities::None;
		uint32 NumPacketProperties = 0;
		FPacketProperty PacketProperties[static_cast<uint32>(EPacketPropertyType::Num_Enumerators)] = {};
		FPacketPropertyHandler PacketPropertyHandlers[static_cast<uint32>(EPacketPropertyType::Num_Enumerators)] = {};

		UINT WintabContextIndex = -1;
		HCTX WintabContextHandle = nullptr;
		uint8 WintabFirstCursor = -1;
		uint8 WintabNumCursors = -1;
	};

	using FTabletContextContainer = Private::TSharedRefDataContainer<FTabletContext, false>;
}
