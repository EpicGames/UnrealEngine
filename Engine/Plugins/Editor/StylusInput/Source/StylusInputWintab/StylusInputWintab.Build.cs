// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StylusInputWintab : ModuleRules
	{
		public StylusInputWintab(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				[
					"Core",
					"ApplicationCore",
					"Slate",
					"SlateCore",
					"StylusInput",
					"Wintab"
				]
			);
		}
	}
}