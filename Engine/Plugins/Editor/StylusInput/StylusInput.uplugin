{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "Stylus & Tablet Plugin",
	"Description" : "Support for advanced stylus and tablet inputs such as pressure, stylus and tablet buttons, and pen angles.",
	"Category" : "Input Devices",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "StylusInput",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "StylusInputDebugWidget",
			"Type" : "EditorNoCommandlet",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "StylusInputMac",
			"Type" : "EditorNoCommandlet",
			"PlatformAllowList": ["Mac"],
			"LoadingPhase" : "PostDefault"
		},
		{
			"Name" : "StylusInputRealTimeStylus",
			"Type" : "EditorNoCommandlet",
			"PlatformAllowList": ["Win64"],
			"LoadingPhase" : "PostDefault"
		},
		{
			"Name" : "StylusInputWintab",
			"Type" : "EditorNoCommandlet",
			"PlatformAllowList": ["Win64"],
			"LoadingPhase" : "PostDefault"
		}
	]
}
